// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Experimental/BuildServerInterface.h"

namespace UE::BuildSelection::Internal
{

struct FArtifact
{
	TSet<FName> Configurations;
	UE::Zen::Build::FBuildServiceInstance::FBuildRecord BuildRecord;
};

void ConformSelection(TSharedPtr<FString>& SelectedItem, const TArray<TSharedPtr<FString>>& SelectionList);

bool WriteSetting(FStringView InSectionName, FStringView InKeyName, FStringView InValue);
bool WriteSetting(FStringView InKeyName, FStringView InValue);
bool ReadSetting(FStringView InSectionName, FStringView InKeyName, FString& OutValue);
bool ReadSetting(FStringView InKeyName, FString& OutValue);

} // namespace UE::Zen::Build