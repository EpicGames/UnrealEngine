// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API STORAGESERVERWIDGETS_API

namespace UE::Zen::Build { class FBuildServiceInstance; }

namespace UE::Zen::Build
{

class FServiceInstanceManager
{
public:
	UE_API TSharedPtr<FBuildServiceInstance> GetBuildServiceInstance() const;
private:
	mutable TSharedPtr<FBuildServiceInstance> CurrentInstance;
};

} // namespace UE::StorageService::Build

#undef UE_API
