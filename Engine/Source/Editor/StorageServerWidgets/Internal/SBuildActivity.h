// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/BuildServerInterface.h"
#include "Experimental/ZenServerInterface.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API STORAGESERVERWIDGETS_API

class SBuildActivityTableRow;

class SBuildActivity : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBuildActivity)
		: _ZenServiceInstance(nullptr)
		, _BuildServiceInstance(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);
	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::Build::FBuildServiceInstance>, BuildServiceInstance);

	SLATE_END_ARGS()

	struct FBuildActivity : public TSharedFromThis<FBuildActivity, ESPMode::ThreadSafe>
	{
		FString Name;
		FString Platform;
		UE::Zen::Build::FBuildServiceInstance::FBuildTransfer Transfer;
	};

	UE_API void Construct(const FArguments& InArgs);

	void AddBuildTransfer(UE::Zen::Build::FBuildServiceInstance::FBuildTransfer Transfer, FStringView Name, FStringView Platform)
	{
		TSharedPtr<SBuildActivity::FBuildActivity, ESPMode::ThreadSafe> NewActivity = MakeShared<FBuildActivity>();
		NewActivity->Name = Name;
		NewActivity->Platform = Platform;
		NewActivity->Transfer = Transfer;
		BuildActivities.Add(NewActivity);
	}

private:
	friend class SBuildActivityTableRow;

	TSharedRef<ITableRow> GenerateBuildActivityRow(TSharedPtr<FBuildActivity> InItem, const TSharedRef<STableViewBase>& InOwningTable);
	TSharedRef<SWidget> GetGridPanel();
	void OnItemDoubleClicked(TSharedPtr<FBuildActivity, ESPMode::ThreadSafe> Item);
	TSharedPtr<SWidget> OnGetBuildActivityContextMenuContent();

	bool CanCancelBuildActivity() const;
	void CancelBuildActivity() const;
	bool CanRetryBuildActivity() const;
	void RetryBuildActivity();
	void OpenDestinationForBuildActivity();
	static void OpenDestinationForBuildActivity(TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance, TSharedPtr<FBuildActivity> Item);
	bool CanClearBuildActivity() const;
	void ClearBuildActivity();
	bool CanViewLogForBuildActivity() const;
	void ViewLogForBuildActivity() const;
	void ClearAllCompleted();

	TSharedPtr<SListView<TSharedPtr<SBuildActivity::FBuildActivity, ESPMode::ThreadSafe>>> BuildActivityListView;
	TArray<TSharedPtr<SBuildActivity::FBuildActivity, ESPMode::ThreadSafe>> BuildActivities;

	SVerticalBox::FSlot* GridSlot = nullptr;
	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
	TAttribute<TSharedPtr<UE::Zen::Build::FBuildServiceInstance>> BuildServiceInstance;
};

typedef TSharedPtr<SBuildActivity::FBuildActivity, ESPMode::ThreadSafe> FBuildActivityPtr;

class SBuildActivityTableRow : public SMultiColumnTableRow<FBuildActivityPtr>
{
public:
	SLATE_BEGIN_ARGS(SBuildActivityTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,
		const TSharedRef<STableViewBase>& InOwnerTableView,
		const FBuildActivityPtr InBuildActivity,
		TSharedPtr<UE::Zen::Build::FBuildServiceInstance> InBuildServiceInstance);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	const FSlateBrush* GetBorder() const;
	FReply OnBrowseClicked();

private:
	FBuildActivityPtr BuildActivity;
	TSharedPtr<UE::Zen::Build::FBuildServiceInstance> BuildServiceInstance;
};

#undef UE_API
