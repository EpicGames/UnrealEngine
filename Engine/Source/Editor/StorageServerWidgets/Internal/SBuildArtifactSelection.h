// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuildSelectionInternal.h"
#include "Experimental/BuildServerInterface.h"
#include "Experimental/ZenServerInterface.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "ZenBuildUtils.h"

#define UE_API STORAGESERVERWIDGETS_API

DECLARE_DELEGATE_FourParams(FOnDownloadWithSpec, const FString& /*ArtifactName*/, const UE::BuildSelection::Internal::FArtifact& /*Artifact*/, const FString& /*DownloadSpecJSON*/, const TArray<FString>& /*FullyMarkedPartNames*/);

class SBuildArtifactSelection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBuildArtifactSelection)
		: _ZenServiceInstance(nullptr)
		, _BuildServiceInstance(nullptr)

	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);
	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::Build::FBuildServiceInstance>, BuildServiceInstance);

	SLATE_EVENT(FOnDownloadWithSpec, OnDownloadWithSpec)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	bool HasArtifactConfigurations() const;
	TSharedPtr<FString> GetSelectedConfiguration() const;
	const TArray<FString>& GetSelectedArtifacts() const;
	UE_API void SetAvailableConfigurations(
			TSet<FString>&& InArtifactConfigurationSet,
			TMultiMap<FString, FString>&& InArtifactNameMap
		);
	UE_API void Refresh(FStringView InNamespace, FStringView InGroupDisplayName, const TMap<FString, UE::BuildSelection::Internal::FArtifact>* InNamedArtifacts, FCbObjectId InHighlightBuildId = {}, bool bInCanViewContents = false);
private:
	void RebuildSelectedGroupArtifactHeader();
	void RebuildSelectedGroupArtifactContents();
	void SetSelectedConfiguration(const TSharedPtr<FString> InSelectedConfiguration);
	TSharedRef<SWidget> CreateArtifactMenuWidget(UE::BuildSelection::Internal::FArtifact* Artifact, const FString& ArtifactName);

	TAttribute<TSharedPtr<UE::Zen::Build::FBuildServiceInstance>> BuildServiceInstance;
	FOnDownloadWithSpec OnDownloadWithSpec;
	FString Namespace;
	bool bCanViewContents = false;

	TMultiMap<FString, FString> ArtifactNameMap;
	TSet<FString> ArtifactConfigurationSet;
	TArray<TSharedPtr<FString>> ArtifactConfigurationList;
	TSharedPtr<FString> SelectedConfiguration;
	TArray<FString> SelectedArtifacts;
	FCbObjectId HighlightedBuildId;
	FString GroupDisplayName;

	TMap<FString, UE::BuildSelection::Internal::FArtifact> NamedArtifacts;

	TSharedPtr<SGridPanel> SelectedGroupArtifactHeaderGrid;
	TSharedPtr<SGridPanel> SelectedGroupArtifactGrid;
};

#undef UE_API
