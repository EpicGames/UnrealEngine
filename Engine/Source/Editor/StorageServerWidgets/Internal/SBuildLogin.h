// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/BuildServerInterface.h"
#include "Experimental/ZenServerInterface.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API STORAGESERVERWIDGETS_API

class SBuildLogin : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBuildLogin)
		: _ZenServiceInstance(nullptr)
		, _BuildServiceInstance(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);
	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::Build::FBuildServiceInstance>, BuildServiceInstance);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	UE::Zen::Build::FBuildServiceInstance::EConnectionState GetEffectiveConnectionState(TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance) const;
	TSharedRef<SWidget> GetGridPanel();

	UE::Zen::Build::FBuildServiceInstance::EConnectionFailureReason LastConnectionFailureReason = UE::Zen::Build::FBuildServiceInstance::EConnectionFailureReason::None;
	bool bTriggeredInteractiveLogin = false;

	SVerticalBox::FSlot* GridSlot = nullptr;
	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
	TAttribute<TSharedPtr<UE::Zen::Build::FBuildServiceInstance>> BuildServiceInstance;
};

#undef UE_API
