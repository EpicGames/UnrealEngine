// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuildSelectionInternal.h"
#include "Experimental/BuildServerInterface.h"
#include "Experimental/ZenServerInterface.h"
#include "Internationalization/Regex.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API STORAGESERVERWIDGETS_API

class FUProjectDictionary;
class SBuildArtifactSelection;
class SMultiSelectComboBox;

namespace UE::Zen::Build
{ 
	struct FListBuildsState;
	struct FBuildReference;
}

DECLARE_DELEGATE_TwoParams(FPreBuildTransfer, FString& /*HostOverride*/, UE::Zen::Build::EBuildTransferRequestFlags& /*RequestFlags*/);

DECLARE_DELEGATE_TwoParams(FPreOplogBuildTransfer, FString& /*HostOverride*/, UE::Zen::Build::EBuildTransferRequestFlags& /*RequestFlags*/);

DECLARE_DELEGATE_ThreeParams(FOnBuildTransferStarted, UE::Zen::Build::FBuildServiceInstance::FBuildTransfer /*Transfer*/,
	FStringView /*Name*/,
	FStringView /*Platform*/);

class SBuildSelection : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBuildSelection)
		: _ZenServiceInstance(nullptr)
		, _BuildServiceInstance(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);
	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::Build::FBuildServiceInstance>, BuildServiceInstance);

	SLATE_EVENT(FPreBuildTransfer, PreBuildTransfer)
	SLATE_EVENT(FPreOplogBuildTransfer, PreOplogBuildTransfer)
	SLATE_EVENT(FOnBuildTransferStarted, OnBuildTransferStarted)
	SLATE_END_ARGS()

	struct FBuildGroup : public TSharedFromThis<FBuildGroup, ESPMode::ThreadSafe>
	{
		FString Namespace;
		FString DisplayName;
		FString CommitIdentifier;
		FDateTime CreatedAt;
		FString Category;
		FString Job;
		TMap<FString, UE::BuildSelection::Internal::FArtifact> NamedArtifacts;
		bool bIsPreflight;
	};

	UE_API void Construct(const FArguments& InArgs);

	enum class EBuildArtifactAction
	{
		None,
		Highlight,
		HighlightAndDownload
	};
	UE_API void ActOnBuildArtifact(EBuildArtifactAction Action, const UE::Zen::Build::FBuildReference& BuildReference, bool bRebuildLists = true);
private:
	enum class EBuildType
	{
		Oplog,
		StagedBuild,
		PackagedBuild,
		EditorPreCompiledBinary,
		EditorInstalledBuild,
		Unknown,
		Count
	};
	struct FKnownBuildType
	{
		EBuildType Type;
		FText UserText;
		int DefaultSelectionOrdering;
	};
	static TPair<FRegexPattern, FKnownBuildType> MakeKnownBuildTypePattern(const FStringView InPattern, EBuildType Type, FText&& InText, int InDefaultSelectionOrdering = 0);
	EBuildType GetSelectedBuildType() const;
	void RebuildLists();
	void RegenerateBuildGroups(UE::Zen::Build::FListBuildsState& ListBuildsState);
	void RefreshPresentedBuildGroups();
	void RegenerateActivePlatformFilters();
	void ValidateBuildGroupSelection();
	TSharedRef<SWidget> OnGenerateTextBlockFromString(TSharedPtr<FString> Item);
	TSharedRef<SWidget> OnGenerateBuildTypeTextBlockFromString(TSharedPtr<FString> Item);
	bool DoesPassAllViewFilters(TSharedPtr<FBuildGroup> InItem) const;
	void PostViewFilterChange();
	bool BuildGroupIsSelectableOrNavigable(TSharedPtr<FBuildGroup> InItem) const;
	TSharedRef<ITableRow> GenerateBuildGroupRow(TSharedPtr<FBuildGroup> InItem, const TSharedRef<STableViewBase>& InOwningTable);
	TSharedPtr<SWidget> OnGetBuildGroupContextMenuContent() const;
	void OnChooseDestinationDirectoryClicked();
	void OnChooseTransferFilterClicked();
	TSharedRef<SWidget> GetBuildDestinationPanel();
	TSharedRef<SWidget> GetGridPanel();
	void RefreshArtifactSelectionWidget(TSharedPtr<FBuildGroup> Item);
	void BuildGroupSelectionChanged(TSharedPtr<FBuildGroup> Item, ESelectInfo::Type SelectInfo);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	void EnforceTargetedBuildActionOnGroup();

	FText ConvertBuildTypeToText(const FString& InBuildType);
	int ConvertBuildTypeToDefaultSelectionOrdering(const FString& InBuildType);

	void SetUserSelectedDestination(const FStringView InDestination);
	FString GetUserSelectedDestination() const;
	FString GetDefaultDestination() const;
	FString GetEffectiveDestination() const;

	void SetUserSelectedAppendBuildNameToDestination(bool bAppendBuildNameToDestination);
	void SetUserSelectedTransferFilter(const FStringView InIncludeFilter, const FStringView InExcludeFilter);

	void SetSelectedStream(const TSharedPtr<FString> InSelectedStream);
	void SetSelectedProject(const TSharedPtr<FString> InSelectedProject);
	void SetSelectedBuildType(const TSharedPtr<FString> InBuildType);
	void SetIncludePreflights(const bool bInIncludePreflights);

	void SetUserSelectedProjectDictionaryRoot(FStringView InRoot);
	void SetFallbackProjectDictionaryRoot(FStringView InRoot);

	static FString SanitizeForPath(const FString& InString);
	static FString SanitizeForZenId(const FString& InString);
	static FString SanitizeBranch(const FString& InString);
	static FString SanitizeBucket(const FString& InString);

	static const TCHAR* LexToString(EBuildType BuildType);

	FReply ExploreDestination_OnClicked();
	void DownloadSelectedArtifactsInSelectedBuildGroup();
	bool DownloadItem(const FBuildGroup& Group, const FString& ArtifactName, const UE::BuildSelection::Internal::FArtifact& Artifact, FString&& DownloadSpecJSONContents = FString(), TArray<FString>&& PartNames = TArray<FString>());

	TMap<FString, FString> EngineInstallations;

	FString FallbackProjectDictionaryRootDir;
	TUniquePtr<FUProjectDictionary> FallbackProjectDictionary;

	FString UserSelectedProjectDictionaryRootDir;
	TUniquePtr<FUProjectDictionary> UserSelectedProjectDictionary;

	FString BranchName;
	int EffectiveCompatibleChangelist = 0;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> ProjectWidget;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> StreamWidget;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> BuildTypeWidget;
	TSharedPtr<SMultiSelectComboBox> PlatformFiltersWidget;

	TSharedPtr<SBuildArtifactSelection> BuildArtifactSelection;

	TArray<TSharedPtr<FString>> ProjectList;
	TArray<TSharedPtr<FString>> StreamList;
	TArray<TSharedPtr<FString>> BuildTypeList;
	TArray<TSharedPtr<FString>> PlatformList;

	struct FTransferFilter
	{
		FString IncludeFilter;
		FString ExcludeFilter;
	};

	mutable FRWLock TargetedBuildIdLock;
	EBuildArtifactAction TargetedBuildIdAction = EBuildArtifactAction::None;
	FCbObjectId TargetedBuildId;

	TArray<FString> ActivePlatformFilters;
	FString UserSelectedDestinations[(int)EBuildType::Count];
	FTransferFilter UserSelectedTransferFilters[(int)EBuildType::Count];
	bool bAppendBuildNameToDestinations[(int)EBuildType::Count];
	bool bCleanDestinations[(int)EBuildType::Count];

	TSharedPtr<FString> SelectedStream;
	TSharedPtr<FString> SelectedProject;
	TSharedPtr<FString> SelectedBuildType;
	FString SelectedTextFilter;
	bool bIncludePreflights = true;

	TArray<TPair<FRegexPattern, FKnownBuildType>> KnownBuildTypePatterns;

	TSharedPtr<SListView<TSharedPtr<SBuildSelection::FBuildGroup, ESPMode::ThreadSafe>>> BuildGroupListView;
	FName BuildGroupSortByColumn;
	EColumnSortMode::Type BuildGroupSortMode;

	std::atomic<uint32> BuildListRefreshesInProgress = 0;
	std::atomic<uint32> BuildRefreshGeneration = 0;
	TArray<TSharedPtr<SBuildSelection::FBuildGroup, ESPMode::ThreadSafe>> BuildGroups;
	TArray<TSharedPtr<SBuildSelection::FBuildGroup, ESPMode::ThreadSafe>> PresentedBuildGroups;

	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
	TAttribute<TSharedPtr<UE::Zen::Build::FBuildServiceInstance>> BuildServiceInstance;
	FPreBuildTransfer PreBuildTransfer;
	FPreOplogBuildTransfer PreOplogBuildTransfer;
	FOnBuildTransferStarted OnBuildTransferStarted;
};

typedef TSharedPtr<SBuildSelection::FBuildGroup, ESPMode::ThreadSafe> FBuildSelectionBuildGroupPtr;

class SBuildGroupTableRow : public SMultiColumnTableRow<FBuildSelectionBuildGroupPtr>
{
public:
	SLATE_BEGIN_ARGS(SBuildGroupTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FBuildSelectionBuildGroupPtr InBuildGroup);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	const FSlateBrush* GetBorder() const;
	FReply OnBrowseClicked();

private:
	FBuildSelectionBuildGroupPtr BuildGroup;
};

#undef UE_API
