// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API STORAGESERVERWIDGETS_API

class SBuildTransferFilters : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBuildTransferFilters)
		: _IncludeFilter(nullptr)
		, _ExcludeFilter(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<FString>, IncludeFilter);
	SLATE_ATTRIBUTE(TSharedPtr<FString>, ExcludeFilter);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FString GetIncludeFilter() const { return IncludeFilter; }
	FString GetExcludeFilter() const { return ExcludeFilter; }

private:
	TSharedRef<SWidget> GetGridPanel();

	FString IncludeFilter;
	FString ExcludeFilter;

	SVerticalBox::FSlot* GridSlot = nullptr;
};

#undef UE_API
