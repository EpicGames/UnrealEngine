// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STORAGESERVERWIDGETS_API

class SWidget;

class SZenStoreStausDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenStoreStausDialog) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:
};

#undef UE_API
