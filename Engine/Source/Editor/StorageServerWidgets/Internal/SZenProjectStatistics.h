// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ZenServerInterface.h"
#include "ZenServiceInstanceManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API STORAGESERVERWIDGETS_API

class SZenProjectStatistics : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenProjectStatistics)
		: _ZenServiceInstance(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	UE_API TSharedRef<SWidget> GetGridPanel();

	UE_API EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);

	SVerticalBox::FSlot* GridSlot = nullptr;
	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
};

#undef UE_API
