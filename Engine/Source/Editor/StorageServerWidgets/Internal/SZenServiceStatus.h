// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ZenServerInterface.h"
#include "ZenServiceInstanceManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API STORAGESERVERWIDGETS_API

class SZenServiceStatus : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenServiceStatus)
		: _ZenServiceInstance(nullptr)
	{ }

	SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:
	struct FState
	{
		UE::Zen::FZenLocalServiceRunContext RunContext;
		FString Version;
		UE::Zen::FGCStatus GCStatus;
		UE::Zen::FZenCacheStats ZenCacheStats;
		UE::Zen::FZenProjectStats ZenProjectStats;
		uint16 LocalPort = 0;
		bool bHaveStats = false;
		bool bGotRunContext = false;
		bool bIsRunning = false;
	};
	static constexpr uint32 NumState = 2;
	FState State[NumState];
	std::atomic<uint32> ActiveStateIndex = 0;


	UE_API const FState& GetCurrentState() const;
	UE_API TSharedRef<SWidget> GetGridPanel();
	UE_API FReply ExploreDataPath_OnClicked();
	UE_API FReply ExploreExecutablePath_OnClicked();

	UE_API EActiveTimerReturnType UpdateState(double InCurrentTime, float InDeltaTime);

	SVerticalBox::FSlot* GridSlot = nullptr;
	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
};

#undef UE_API
