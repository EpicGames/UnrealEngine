// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ZenServerInterface.h"
#include "ZenServiceInstanceManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SExpandableArea.h"

class SZenWorkspaces : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenWorkspaces)
		: _ZenServiceInstance(nullptr)
		{
		}

		SLATE_ATTRIBUTE(TSharedPtr<UE::Zen::FZenServiceInstance>, ZenServiceInstance);
		SLATE_ATTRIBUTE(float, UpdateFrequency);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	int32 GetWorkspaceCount() const { return Workspaces.ZenWorkspaces.Num(); }
	void UpdateWorkspaces(float InDeltaTime, bool bForce = false);

private:
	TSharedRef<SWidget> GetWorkspaceList();

	FReply ZenWorkspacesArea_Toggle() const;

	FReply OnBackToMainWidget();
	FReply OnCreateNewWorkspaceClicked();

	const FSlateBrush* ZenWorkspacesArea_Icon() const;

private:
	TAttribute<TSharedPtr<UE::Zen::FZenServiceInstance>> ZenServiceInstance;
	float UpdateFrequency = 0.f;
	TSharedPtr<SVerticalBox> WorkspaceArea;
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;
	SVerticalBox::FSlot* GridSlot = nullptr;
	TSharedPtr<SExpandableArea> Expandable;

	TArray<TSharedPtr<UE::Zen::FZenWorkspaces>> WorkspaceModels;
	TSharedPtr<SListView<TSharedPtr<UE::Zen::FZenWorkspaces>>> WorkspacesListView;

	UE::Zen::FZenWorkspaces Workspaces;
};
