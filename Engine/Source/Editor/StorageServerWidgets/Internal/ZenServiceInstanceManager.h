// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API STORAGESERVERWIDGETS_API

namespace UE::Zen { class FZenServiceInstance; }

namespace UE::Zen
{

class FServiceInstanceManager
{
public:
	UE_API TSharedPtr<FZenServiceInstance> GetZenServiceInstance() const;
private:
	mutable uint16 CurrentPort = 0;
	mutable TSharedPtr<FZenServiceInstance> CurrentInstance;
};

} // namespace UE::Zen

#undef UE_API
