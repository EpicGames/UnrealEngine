// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildSelectionInternal.h"
#include "Misc/Paths.h"

namespace UE::BuildSelection::Internal
{

void ConformSelection(TSharedPtr<FString>& SelectedItem, const TArray<TSharedPtr<FString>>& SelectionList)
{
	if (!SelectedItem)
	{
		SelectedItem = SelectionList.IsEmpty() ? nullptr : SelectionList[0];
		return;
	}

	const TSharedPtr<FString>* FoundSelectionListItem = SelectionList.FindByPredicate([&SelectedItem](const TSharedPtr<FString>& Item)
		{
			return *Item == *SelectedItem;
		});

	SelectedItem = FoundSelectionListItem ? *FoundSelectionListItem : SelectionList.IsEmpty() ? nullptr : SelectionList[0];
}

bool WriteSetting(const FStringView InSectionName, const FStringView InKeyName, const FStringView InValue)
{
	static const FString SectionName = TEXT("Unreal Engine/Build Storage");
	const FString FinalSectionName = InSectionName.IsEmpty() ? SectionName : FPaths::Combine(SectionName, InSectionName);
	const FString FinalKeyName(InKeyName);
	FString Value(InValue);
	return FPlatformMisc::SetStoredValue(TEXT("Epic Games"), FinalSectionName, FinalKeyName, Value);
}

bool WriteSetting(const FStringView InKeyName, const FStringView InValue)
{
	return WriteSetting(TEXT(""), InKeyName, InValue);
}

bool ReadSetting(const FStringView InSectionName, const FStringView InKeyName, FString& OutValue)
{
	static const FString SectionName = TEXT("Unreal Engine/Build Storage");
	const FString FinalSectionName = InSectionName.IsEmpty() ? SectionName : FPaths::Combine(SectionName, InSectionName);
	const FString FinalKeyName(InKeyName);
	return FPlatformMisc::GetStoredValue(TEXT("Epic Games"), FinalSectionName, FinalKeyName, OutValue);
}

bool ReadSetting(const FStringView InKeyName, FString& OutValue)
{
	return ReadSetting(TEXT(""), InKeyName, OutValue);
}

} // namespace UE::Zen::Build
