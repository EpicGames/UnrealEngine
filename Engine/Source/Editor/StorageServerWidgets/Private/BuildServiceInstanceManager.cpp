// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildServiceInstanceManager.h"
#include "Experimental/BuildServerInterface.h"


namespace UE::Zen::Build
{

TSharedPtr<FBuildServiceInstance> FServiceInstanceManager::GetBuildServiceInstance() const
{
	if (!CurrentInstance)
	{
		CurrentInstance = MakeShared<FBuildServiceInstance>();
	}
	return CurrentInstance;
}

} // namespace UE::StorageService::Build
