// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildActivity.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "ZenServiceInstanceManager.h"

#define LOCTEXT_NAMESPACE "StorageServerBuild"

namespace UE::BuildActivity::Internal
{
	namespace FBuildActivityIds
	{
		const FName ColStatus = TEXT("Status");
		const FName ColName = TEXT("Name");
		const FName ColPlatform = TEXT("Platform");
		const FName ColDuration = TEXT("Duration");
		const FName ColDescription = TEXT("Description");
		const FName ColActions = TEXT("Actions");
	}
}

void SBuildActivity::Construct(const FArguments& InArgs)
{
	ZenServiceInstance = InArgs._ZenServiceInstance;
	BuildServiceInstance = InArgs._BuildServiceInstance;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];
}

TSharedRef<ITableRow> SBuildActivity::GenerateBuildActivityRow(FBuildActivityPtr InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	return SNew(SBuildActivityTableRow, InOwningTable, InItem, BuildServiceInstance.Get());
}

TSharedRef<SWidget> SBuildActivity::GetGridPanel()
{
	using namespace UE::BuildActivity::Internal;

	TSharedRef<SVerticalBox> Panel = SNew(SVerticalBox)
	.Visibility_Lambda([this]()
	{
		return BuildActivities.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	});

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	Panel->AddSlot()
	.AutoHeight()
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("BuildActivity_Activity", "Activity"))
	];

	Panel->AddSlot()
	.AutoHeight()
	.Padding(FMargin(ColumnMargin, RowMargin))
	[
		SAssignNew(BuildActivityListView, SListView<FBuildActivityPtr>)
			.ListItemsSource(&BuildActivities)
			.OnGenerateRow(this, &SBuildActivity::GenerateBuildActivityRow)
			.OnMouseButtonDoubleClick(this, &SBuildActivity::OnItemDoubleClicked)
			.OnContextMenuOpening(this, &SBuildActivity::OnGetBuildActivityContextMenuContent)
			.HeaderRow
			(
				SNew(SHeaderRow)
				.Visibility(EVisibility::Collapsed)
				+ SHeaderRow::Column(FBuildActivityIds::ColStatus).DefaultLabel(LOCTEXT("BuildActivity_ActivityStatus", "Status"))
					.FillWidth(0.05f)
				+ SHeaderRow::Column(FBuildActivityIds::ColName).DefaultLabel(LOCTEXT("BuildActivity_ActivityName", "Name"))
					.FillWidth(0.2f).HAlignCell(HAlign_Left).HAlignHeader(HAlign_Left).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FBuildActivityIds::ColPlatform).DefaultLabel(LOCTEXT("BuildActivity_ActivityPlatform", "Platform"))
					.FillWidth(0.15f).HAlignCell(HAlign_Left).HAlignHeader(HAlign_Left).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FBuildActivityIds::ColDuration).DefaultLabel(LOCTEXT("BuildActivity_ActivityDuration", "Duration"))
					.FillWidth(0.05f).HAlignCell(HAlign_Left).HAlignHeader(HAlign_Left).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FBuildActivityIds::ColDescription).DefaultLabel(LOCTEXT("BuildActivity_ActivityDescription", "Description"))
					.FillWidth(0.49f).HAlignCell(HAlign_Fill).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FBuildActivityIds::ColActions).DefaultLabel(LOCTEXT("BuildActivity_ActivityActions", "Actions"))
					.FillWidth(0.06f).HAlignCell(HAlign_Fill).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
			)
	];


	return Panel;
}

void SBuildActivity::OnItemDoubleClicked(TSharedPtr<FBuildActivity, ESPMode::ThreadSafe> Item)
{
	OpenDestinationForBuildActivity(BuildServiceInstance.Get(), Item);
}

TSharedPtr<SWidget> SBuildActivity::OnGetBuildActivityContextMenuContent()
{
	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();

	if (SelectedItems.IsEmpty())
	{
		return nullptr;
	}

	const bool bCloseAfterSelection = true;
	const bool bCloseSelfOnly = false;
	const bool bSearchable = false;
	const bool bRecursivelySearchable = false;
	FMenuBuilder MenuBuilder(bCloseAfterSelection,
		nullptr,
		TSharedPtr<FExtender>(),
		bCloseSelfOnly,
		&FCoreStyle::Get(),
		bSearchable,
		NAME_None,
		bRecursivelySearchable);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_OpenDestination", "Open destination"),
		LOCTEXT("BuildActivity_OpenDestinationTooltip", "Open the destination directory or page for the activity"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::OpenDestinationForBuildActivity)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_ViewLog", "View log"),
		LOCTEXT("BuildActivity_ViewLogTooltip", "View log file for the activity"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::ViewLogForBuildActivity),
			FCanExecuteAction::CreateRaw(this, &SBuildActivity::CanViewLogForBuildActivity)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_Cancel", "Cancel"),
		LOCTEXT("BuildActivity_CancelTooltip", "Cancel the pending or in progress activity"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::CancelBuildActivity),
			FCanExecuteAction::CreateRaw(this, &SBuildActivity::CanCancelBuildActivity)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_Retry", "Retry"),
		LOCTEXT("BuildActivity_RetryTooltip", "Retry the failed or canceled activity"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::RetryBuildActivity),
			FCanExecuteAction::CreateRaw(this, &SBuildActivity::CanRetryBuildActivity)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_Clear", "Clear"),
		LOCTEXT("BuildActivity_ClearTooltip", "Clear this activity if it is completed"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::ClearBuildActivity),
			FCanExecuteAction::CreateRaw(this, &SBuildActivity::CanClearBuildActivity)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildActivity_ClearAllCompleted", "Clear all completed"),
		LOCTEXT("BuildActivity_ClearAllCompletedTooltip", "Clear all completed items"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SBuildActivity::ClearAllCompleted)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	return MenuBuilder.MakeWidget();
}

bool SBuildActivity::CanCancelBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return false;
	}
	FBuildServiceInstance::EBuildTransferStatus Status = SelectedItems[0]->Transfer.GetStatus();
	return (Status == FBuildServiceInstance::EBuildTransferStatus::Queued) || (Status == FBuildServiceInstance::EBuildTransferStatus::Active);
}

void SBuildActivity::CancelBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}
	SelectedItems[0]->Transfer.RequestCancel();
}

bool SBuildActivity::CanRetryBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return false;
	}
	FBuildServiceInstance::EBuildTransferStatus Status = SelectedItems[0]->Transfer.GetStatus();
	return (Status == FBuildServiceInstance::EBuildTransferStatus::Canceled) || (Status == FBuildServiceInstance::EBuildTransferStatus::Failed);
}

void SBuildActivity::RetryBuildActivity()
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}
	if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		AddBuildTransfer(ServiceInstance->RepeatBuildTransfer(SelectedItems[0]->Transfer),
			SelectedItems[0]->Name, SelectedItems[0]->Platform);
	}
}

void SBuildActivity::OpenDestinationForBuildActivity()
{
	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}
	OpenDestinationForBuildActivity(BuildServiceInstance.Get(), SelectedItems[0]);
}

void SBuildActivity::OpenDestinationForBuildActivity(TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance, TSharedPtr<FBuildActivity> Item)
{
	using namespace UE::Zen::Build;

	if (ServiceInstance)
	{
		switch (Item->Transfer.GetType())
		{
		case FBuildServiceInstance::FBuildTransfer::EType::Oplog:
			{
				const FString Separator(TEXT("/"));
				FString ProjectId;
				FString OplogId;
				if (Item->Transfer.GetDestination().Split(Separator, &ProjectId, &OplogId, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					UE::Zen::FZenLocalServiceRunContext RunContext;
					uint16 LocalPort = 8558;
					if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
					{
						if (!UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort))
						{
							UE::Zen::StartLocalService(RunContext);
							UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort);
						}
					}
					FPlatformProcess::LaunchURL(*FString::Printf(TEXT("http://localhost:%d/dashboard/?page=oplog&project=%s&oplog=%s"), LocalPort, *ProjectId, *OplogId), nullptr, nullptr);
				}
			}
			break;
		case FBuildServiceInstance::FBuildTransfer::EType::Files:
			{
				FString FolderToExplore = Item->Transfer.GetDestination();
				for (int Index = 0; Index < 3; ++Index)
				{
					if (FolderToExplore.IsEmpty() || FPaths::DirectoryExists(FolderToExplore))
					{
						break;
					}
					FolderToExplore = FPaths::GetPath(FolderToExplore);
				}
				if (!FolderToExplore.IsEmpty() && FPaths::DirectoryExists(FolderToExplore))
				{
					FPlatformProcess::ExploreFolder(*FolderToExplore);
				}
			}
			break;
		}
	}
}

bool SBuildActivity::CanClearBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return false;
	}
	FBuildServiceInstance::EBuildTransferStatus Status = SelectedItems[0]->Transfer.GetStatus();
	return (Status != FBuildServiceInstance::EBuildTransferStatus::Queued) && (Status != FBuildServiceInstance::EBuildTransferStatus::Active);
}

void SBuildActivity::ClearBuildActivity()
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}
	BuildActivities.Remove(SelectedItems[0]);
}

bool SBuildActivity::CanViewLogForBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return false;
	}
	FString LogFilename = SelectedItems[0]->Transfer.GetLogFilename();
	return !LogFilename.IsEmpty() && FPaths::FileExists(LogFilename);
}

void SBuildActivity::ViewLogForBuildActivity() const
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> SelectedItems = BuildActivityListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}
	FString LogFilename = SelectedItems[0]->Transfer.GetLogFilename();
	if (!LogFilename.IsEmpty() && FPaths::FileExists(LogFilename))
	{
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*LogFilename);
	}
}

void SBuildActivity::ClearAllCompleted()
{
	using namespace UE::Zen::Build;

	TArray<FBuildActivityPtr> ActivitiesToRemove;
	for (FBuildActivityPtr BuildActivity : BuildActivities)
	{
		FBuildServiceInstance::EBuildTransferStatus Status = BuildActivity->Transfer.GetStatus();
		if ((Status == FBuildServiceInstance::EBuildTransferStatus::Failed) ||
			(Status == FBuildServiceInstance::EBuildTransferStatus::Canceled) ||
			(Status == FBuildServiceInstance::EBuildTransferStatus::Succeeded))
		{
			ActivitiesToRemove.AddUnique(BuildActivity);
		}
	}

	for (FBuildActivityPtr BuildActivity : ActivitiesToRemove)
	{
		BuildActivities.Remove(BuildActivity);
	}
}

void
SBuildActivityTableRow::Construct(const FArguments& InArgs,
	const TSharedRef<STableViewBase>& InOwnerTableView,
	const FBuildActivityPtr InBuildActivity,
	TSharedPtr<UE::Zen::Build::FBuildServiceInstance> InBuildServiceInstance)
{
	BuildActivity = InBuildActivity;
	BuildServiceInstance = InBuildServiceInstance;

	SMultiColumnTableRow<FBuildActivityPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget>
SBuildActivityTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace UE::Zen::Build;
	using namespace UE::BuildActivity::Internal;

	if (ColumnName == FBuildActivityIds::ColStatus)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(1.f,1.f)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
					.Text_Lambda([this]()
						{
							FString Status;
							if (BuildServiceInstance)
							{
								switch(BuildActivity->Transfer.GetStatus())
								{
								case FBuildServiceInstance::EBuildTransferStatus::Invalid:
									return LOCTEXT("BuildActivity_TransferStatusInvalid", "Invalid");
								case FBuildServiceInstance::EBuildTransferStatus::Queued:
									return LOCTEXT("BuildActivity_TransferStatusQueued", "Queued");
								case FBuildServiceInstance::EBuildTransferStatus::Active:
									return LOCTEXT("BuildActivity_TransferStatusActive", "Active");
								case FBuildServiceInstance::EBuildTransferStatus::Failed:
									return LOCTEXT("BuildActivity_TransferStatusFailed", "Failed");
								case FBuildServiceInstance::EBuildTransferStatus::Canceled:
									return LOCTEXT("BuildActivity_TransferStatusCanceled", "Canceled");
								case FBuildServiceInstance::EBuildTransferStatus::Succeeded:
									return LOCTEXT("BuildActivity_TransferStatusSucceeded", "Succeeded");
								}
							}
							return LOCTEXT("BuildActivity_TransferStatusUnknown", "Unknown");
						})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f,1.f,1.f,1.f)
				[
					SNew(SCircularThrobber)
					.Radius(6.f)
					.Visibility_Lambda([this]()
					{
						return BuildActivity->Transfer.GetStatus() == FBuildServiceInstance::EBuildTransferStatus::Active
							? EVisibility::Visible : EVisibility::Collapsed;
					})
				];
	}
	else if (ColumnName == FBuildActivityIds::ColName)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
					.Text(FText::FromString(BuildActivity->Name))
				];
	}
	else if (ColumnName == FBuildActivityIds::ColPlatform)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
					.Text(FText::FromString(BuildActivity->Platform))
				];
	}
	else if (ColumnName == FBuildActivityIds::ColDuration)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(1.f, 1.f)
			[
				SNew(STextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
					.Text_Lambda([this]()
						{
							UE::FMonotonicTimeSpan Span(BuildActivity->Transfer.GetTransferDuration());
							if (Span.IsZero())
							{
								return FText::FromString(FString());
							}
							else
							{
								return FText::AsTimespan(FTimespan::FromSeconds(Span.ToSeconds()));
							}
						})
			];
	}
	else if (ColumnName == FBuildActivityIds::ColDescription)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						FString Label;
						FString Detail;
						float Percent;
						if (BuildActivity->Transfer.GetCurrentProgress(Label, Detail, Percent))
						{
							return FText::FromString(Detail.IsEmpty() ? Label : Detail);
						}
						return FText::FromString(FString());
					})
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
					.Justification(ETextJustify::Type::Left)
					.WrapTextAt(0.f)
					.ToolTipText_Lambda([this]()
					{
						return FText::FromString(BuildActivity->Transfer.GetRecentOutput());
					})
					.Visibility_Lambda([this]()
					{
						return BuildActivity->Transfer.GetStatus() == FBuildServiceInstance::EBuildTransferStatus::Active ? EVisibility::Visible : EVisibility::Collapsed;
					})
				];
	}
	else if (ColumnName == FBuildActivityIds::ColActions)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.f,1.f)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
					.ToolTipText(LOCTEXT("BuildActivity_OpenDestinationButtonTooltip", "Open the destination directory or page for the activity"))
					.OnClicked_Lambda([this]
					{
						SBuildActivity::OpenDestinationForBuildActivity(BuildServiceInstance, BuildActivity);
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.Image_Lambda([this]
						{
							return BuildActivity->Transfer.GetType() == FBuildServiceInstance::FBuildTransfer::EType::Oplog ? FAppStyle::Get().GetBrush("Zen.BrowserView") : FAppStyle::Get().GetBrush("Zen.FolderView");
						})
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.f,1.f)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
					.ToolTipText(LOCTEXT("BuildActivity_CancelButtonTooltip", "Cancel the pending or in progress activity"))
					.IsEnabled_Lambda([this]
					{
						FBuildServiceInstance::EBuildTransferStatus Status = BuildActivity->Transfer.GetStatus();
						if (Status == FBuildServiceInstance::EBuildTransferStatus::Active || Status == FBuildServiceInstance::EBuildTransferStatus::Queued)
						{
							return true;
						}
						return false;
					})
					.OnClicked_Lambda([this]
					{
						BuildActivity->Transfer.RequestCancel();
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Zen.Cancel"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				];
	}

	return SNullWidget::NullWidget;
}

const FSlateBrush*
SBuildActivityTableRow::GetBorder() const
{
	return STableRow<FBuildActivityPtr>::GetBorder();
}

FReply
SBuildActivityTableRow::OnBrowseClicked()
{
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
