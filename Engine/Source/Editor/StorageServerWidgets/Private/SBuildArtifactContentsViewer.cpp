// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildArtifactContentsViewer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Serialization/JsonWriter.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SThrobber.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "BuildArtifactContents"

namespace UE::BuildArtifactContents::Internal
{
	const FName ColMark = TEXT("Mark");
	const FName ColName = TEXT("Name");
	const FName ColSize = TEXT("Size");
}

// ---------------------------------------------------------------------------
// FPartTab
// ---------------------------------------------------------------------------

ECheckBoxState FPartTab::GetCheckState() const
{
	if (MarkedFileCount == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	if (MarkedFileCount == TotalFileCount)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Undetermined;
}

bool FPartTab::IsFullyMarked() const
{
	return TotalFileCount > 0 && MarkedFileCount == TotalFileCount;
}

// ---------------------------------------------------------------------------
// SContentsEntryRow
// ---------------------------------------------------------------------------

void SContentsEntryRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FContentsEntry> InEntry)
{
	Entry = InEntry;
	OnMarkChanged = InArgs._OnMarkChanged;
	SMultiColumnTableRow<TSharedPtr<FContentsEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SContentsEntryRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace UE::BuildArtifactContents::Internal;

	if (ColumnName == ColMark)
	{
		return SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{
				return Entry->bIsMarked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				Entry->bIsMarked = (NewState == ECheckBoxState::Checked);
				OnMarkChanged.ExecuteIfBound();
			});
	}
	else if (ColumnName == ColName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Entry->Name))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
				.Margin(FMargin(4.0f, 2.0f))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SCircularThrobber)
				.Radius(7.f)
				.NumPieces(6)
				.Period(0.75f)
				.Visibility_Lambda([this]()
				{
					return Entry->bIsDownloading ? EVisibility::Visible : EVisibility::Collapsed;
				})
			];
	}
	else if (ColumnName == ColSize)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(SBuildArtifactContentsViewer::FormatSize(Entry->RawSize)))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
			.Justification(ETextJustify::Right)
			.Margin(FMargin(4.0f, 2.0f));
	}
	return SNullWidget::NullWidget;
}

// ---------------------------------------------------------------------------
// SBuildArtifactContentsViewer
// ---------------------------------------------------------------------------

SBuildArtifactContentsViewer::~SBuildArtifactContentsViewer()
{
	for (FPendingFileOpen& Pending : PendingFileOpens)
	{
		Pending.Transfer.RequestCancel();
	}
}

void SBuildArtifactContentsViewer::Construct(const FArguments& InArgs)
{
	using namespace UE::BuildArtifactContents::Internal;

	OnStartFileDownload = InArgs._OnStartFileDownload;

	// Initialize tab button styles from the docking tab style
	{
		const FDockTabStyle& DockTabStyle = FCoreStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");

		InactiveTabButtonStyle = FButtonStyle()
			.SetNormal(DockTabStyle.NormalBrush)
			.SetHovered(DockTabStyle.HoveredBrush)
			.SetPressed(DockTabStyle.ForegroundBrush)
			.SetNormalForeground(DockTabStyle.NormalForegroundColor)
			.SetHoveredForeground(DockTabStyle.HoveredForegroundColor)
			.SetPressedForeground(DockTabStyle.ActiveForegroundColor)
			.SetNormalPadding(DockTabStyle.TabPadding)
			.SetPressedPadding(DockTabStyle.TabPadding);

		ActiveTabButtonStyle = FButtonStyle()
			.SetNormal(DockTabStyle.ForegroundBrush)
			.SetHovered(DockTabStyle.ForegroundBrush)
			.SetPressed(DockTabStyle.ForegroundBrush)
			.SetNormalForeground(DockTabStyle.ActiveForegroundColor)
			.SetHoveredForeground(DockTabStyle.ActiveForegroundColor)
			.SetPressedForeground(DockTabStyle.ActiveForegroundColor)
			.SetNormalPadding(DockTabStyle.TabPadding)
			.SetPressedPadding(DockTabStyle.TabPadding);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Header
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f)
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("ContentsHeader", "Contents for: {0}"), FText::FromString(InArgs._ArtifactName)))
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
		]

		// Search box + Mark All / Clear All buttons
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Filter files (wildcards: * ? ; separated)"))
				.OnTextChanged(this, &SBuildArtifactContentsViewer::OnSearchTextChanged)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("MarkAll", "Mark All"))
				.IsEnabled_Lambda([this]()
				{
					return MarkedFileCount < TotalFileCount;
				})
				.OnClicked_Lambda([this]()
				{
					MarkAllFiles();
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearAll", "Clear All"))
				.IsEnabled_Lambda([this]()
				{
					return MarkedFileCount > 0;
				})
				.OnClicked_Lambda([this]()
				{
					ClearAllMarkedFiles();
					return FReply::Handled();
				})
			]
		]

		// Tab bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 0.0f)
		[
			SAssignNew(TabBarScrollBox, SScrollBox)
			.Orientation(EOrientation::Orient_Horizontal)
		]

		// Contents area with widget switcher
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(4.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				SAssignNew(ContentSwitcher, SWidgetSwitcher)
			]
		]

		// Status bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 4.0f)
		[
			SAssignNew(StatusText, STextBlock)
			.Text(LOCTEXT("Loading", "Loading..."))
			.Font(FCoreStyle::GetDefaultFontStyle("Italic", 9))
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor() * 0.6f)
		]

		// Marked files status
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 0.0f, 8.0f, 4.0f)
		[
			SAssignNew(MarkedStatusText, STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Italic", 9))
			.ColorAndOpacity(FStyleColors::AccentYellow)
			.Visibility_Lambda([this]()
			{
				return MarkedFileCount > 0 ? EVisibility::Visible : EVisibility::Collapsed;
			})
		]
	];

	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SBuildArtifactContentsViewer::PollPendingContents));
}

void SBuildArtifactContentsViewer::QueueContents(TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>&& BuildContents)
{
	FScopeLock Lock(&PendingContentsMutex);
	PendingContents.Emplace(MoveTemp(BuildContents));
}

EActiveTimerReturnType SBuildArtifactContentsViewer::PollPendingContents(double InCurrentTime, float InDeltaTime)
{
	TOptional<TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>> Contents;
	{
		FScopeLock Lock(&PendingContentsMutex);
		if (PendingContents.IsSet())
		{
			Contents.Emplace(MoveTemp(PendingContents.GetValue()));
			PendingContents.Reset();
		}
	}

	if (Contents.IsSet())
	{
		SetContents(Contents.GetValue());
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SBuildArtifactContentsViewer::SetContents(TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>& BuildContents)
{
	using namespace UE::BuildArtifactContents::Internal;

	PartTabs.Empty();
	TotalFileCount = 0;
	TotalFileSize = 0;

	// Sort parts by name
	TArray<FString> PartNames;
	BuildContents.GetKeys(PartNames);
	PartNames.Sort();

	// Reserve so that PartTabs elements (and their PartName strings) stay at
	// stable addresses while we populate FContentsEntry::PartName views below.
	PartTabs.Reserve(PartNames.Num());

	for (const FString& PartName : PartNames)
	{
		UE::Zen::Build::FBuildServiceInstance::FBuildPart& Part = BuildContents[PartName];
		FPartTab& Tab = PartTabs.AddDefaulted_GetRef();
		Tab.PartName = PartName;
		Tab.PartId = Part.Id;

		// Sort files by name
		Part.Files.Sort([](const UE::Zen::Build::FBuildServiceInstance::FBuildFile& A, const UE::Zen::Build::FBuildServiceInstance::FBuildFile& B)
		{
			return A.Name < B.Name;
		});

		for (const UE::Zen::Build::FBuildServiceInstance::FBuildFile& File : Part.Files)
		{
			TSharedPtr<FContentsEntry> FileEntry = MakeShared<FContentsEntry>();
			FileEntry->Name = File.Name;
			FileEntry->PartName = Tab.PartName;
			FileEntry->PartId = Part.Id;
			FileEntry->RawSize = File.RawSize;
			Tab.AllFiles.Add(FileEntry);

			Tab.TotalFileSize += File.RawSize;
			Tab.TotalFileCount++;
		}

		TotalFileSize += Tab.TotalFileSize;
		TotalFileCount += Tab.TotalFileCount;
	}

	// Build list views and add to content switcher
	if (ContentSwitcher.IsValid())
	{
		for (int32 TabIndex = 0; TabIndex < PartTabs.Num(); ++TabIndex)
		{
			FPartTab& Tab = PartTabs[TabIndex];
			Tab.FilteredFiles = Tab.AllFiles;

			ContentSwitcher->AddSlot()
			[
				SAssignNew(Tab.ListView, SListView<TSharedPtr<FContentsEntry>>)
				.ListItemsSource(&Tab.FilteredFiles)
				.OnGenerateRow(this, &SBuildArtifactContentsViewer::OnGenerateRow)
				.OnContextMenuOpening(this, &SBuildArtifactContentsViewer::OnContextMenuOpening)
				.SelectionMode(ESelectionMode::Multi)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(ColMark)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(24.0f)
					.HAlignHeader(HAlign_Center)
					.HAlignCell(HAlign_Center)

					+ SHeaderRow::Column(ColName)
					.DefaultLabel(LOCTEXT("NameColumn", "Name"))
					.FillWidth(0.85f)
					.SortMode(this, &SBuildArtifactContentsViewer::GetColumnSortMode, ColName)
					.OnSort(this, &SBuildArtifactContentsViewer::OnColumnSortModeChanged)

					+ SHeaderRow::Column(ColSize)
					.DefaultLabel(LOCTEXT("SizeColumn", "Size"))
					.FillWidth(0.15f)
					.HAlignCell(HAlign_Right)
					.SortMode(this, &SBuildArtifactContentsViewer::GetColumnSortMode, ColSize)
					.OnSort(this, &SBuildArtifactContentsViewer::OnColumnSortModeChanged)
				)
			];
		}
	}

	// Build tab bar
	RebuildTabBar();

	// Activate first tab
	ActiveTabIndex = 0;
	if (ContentSwitcher.IsValid() && PartTabs.Num() > 0)
	{
		ContentSwitcher->SetActiveWidgetIndex(0);
	}

	bIsLoading = false;

	SortAndRefreshList();
}

void SBuildArtifactContentsViewer::RebuildTabBar()
{
	if (!TabBarScrollBox.IsValid())
	{
		return;
	}

	TabBarScrollBox->ClearChildren();

	for (int32 TabIndex = 0; TabIndex < PartTabs.Num(); ++TabIndex)
	{
		FPartTab& Tab = PartTabs[TabIndex];
		const bool bIsActive = (TabIndex == ActiveTabIndex);

		TabBarScrollBox->AddSlot()
		.Padding(0.0f, 0.0f, 2.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SButton)
				.ButtonStyle(bIsActive ? &ActiveTabButtonStyle : &InactiveTabButtonStyle)
				.OnClicked_Lambda([this, TabIndex]()
				{
					SetActiveTab(TabIndex);
					return FReply::Handled();
				})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this, TabIndex]()
						{
							if (PartTabs.IsValidIndex(TabIndex))
							{
								return PartTabs[TabIndex].GetCheckState();
							}
							return ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([this, TabIndex](ECheckBoxState NewState)
						{
							if (!PartTabs.IsValidIndex(TabIndex))
							{
								return;
							}
							FPartTab& ClickedTab = PartTabs[TabIndex];
							// Indeterminate or unchecked -> check all; checked -> uncheck all
							const bool bMark = (NewState != ECheckBoxState::Unchecked) || (ClickedTab.GetCheckState() == ECheckBoxState::Undetermined);
							for (const TSharedPtr<FContentsEntry>& Entry : ClickedTab.AllFiles)
							{
								Entry->bIsMarked = bMark;
							}
							RecalculateMarkedCounts();
						})
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(Tab.PartName))
						.Font(bIsActive
							? FCoreStyle::GetDefaultFontStyle("Bold", 9)
							: FCoreStyle::GetDefaultFontStyle("Regular", 9))
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(2.f)
				.Visibility(bIsActive ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed)
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FStyleColors::Primary)
					.Padding(0)
				]
			]
		];
	}
}

void SBuildArtifactContentsViewer::SetActiveTab(int32 Index)
{
	if (!PartTabs.IsValidIndex(Index))
	{
		return;
	}

	ActiveTabIndex = Index;
	if (ContentSwitcher.IsValid())
	{
		ContentSwitcher->SetActiveWidgetIndex(Index);
	}

	// Rebuild tab bar to update active/inactive visual state
	RebuildTabBar();

	// Refresh status text for the newly active tab
	RefreshFilteredList();
}

TSharedRef<ITableRow> SBuildArtifactContentsViewer::OnGenerateRow(TSharedPtr<FContentsEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SContentsEntryRow, OwnerTable, Entry)
		.OnMarkChanged(FSimpleDelegate::CreateSP(this, &SBuildArtifactContentsViewer::RecalculateMarkedCounts));
}

FString SBuildArtifactContentsViewer::FormatSize(uint64 SizeInBytes)
{
	const FCulturePtr LocaleCulture = FInternationalization::Get().GetCulture(FPlatformMisc::GetDefaultLocale());

	if (SizeInBytes < 1024)
	{
		return FString::Printf(TEXT("%llu B"), SizeInBytes);
	}

	const TCHAR* Suffix;
	double Value;
	int32 FractionalDigits;

	if (SizeInBytes < 1024 * 1024)
	{
		Value = (double)SizeInBytes / 1024.0;
		Suffix = TEXT("KB");
		FractionalDigits = 1;
	}
	else if (SizeInBytes < 1024ull * 1024 * 1024)
	{
		Value = (double)SizeInBytes / (1024.0 * 1024.0);
		Suffix = TEXT("MB");
		FractionalDigits = 1;
	}
	else
	{
		Value = (double)SizeInBytes / (1024.0 * 1024.0 * 1024.0);
		Suffix = TEXT("GB");
		FractionalDigits = 2;
	}

	const FNumberFormattingOptions Options = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMinimumFractionalDigits(FractionalDigits)
		.SetMaximumFractionalDigits(FractionalDigits);

	return FText::AsNumber(Value, &Options, LocaleCulture).ToString() + TEXT(" ") + Suffix;
}

void SBuildArtifactContentsViewer::SortAndRefreshList()
{
	using namespace UE::BuildArtifactContents::Internal;

	if (SortMode != EColumnSortMode::None && !SortByColumn.IsNone())
	{
		const bool bAscending = (SortMode == EColumnSortMode::Ascending);

		for (FPartTab& Tab : PartTabs)
		{
			Tab.AllFiles.Sort([&](const TSharedPtr<FContentsEntry>& A, const TSharedPtr<FContentsEntry>& B)
			{
				if (SortByColumn == ColName)
				{
					return bAscending ? (A->Name < B->Name) : (A->Name > B->Name);
				}
				else if (SortByColumn == ColSize)
				{
					return bAscending ? (A->RawSize < B->RawSize) : (A->RawSize > B->RawSize);
				}
				return false;
			});
		}
	}

	RefreshFilteredList();
}

void SBuildArtifactContentsViewer::RefreshFilteredList()
{
	TArray<FString> IncludeWildcards;
	if (!SearchText.IsEmpty())
	{
		ParseIncludeWildcards(SearchText.ToString(), IncludeWildcards);
	}

	int32 FilteredFileCount = 0;
	uint64 FilteredFileSize = 0;

	for (FPartTab& Tab : PartTabs)
	{
		Tab.FilteredFiles.Empty();

		if (SearchText.IsEmpty())
		{
			Tab.FilteredFiles = Tab.AllFiles;
			FilteredFileCount += Tab.TotalFileCount;
			FilteredFileSize += Tab.TotalFileSize;
		}
		else
		{
			for (const TSharedPtr<FContentsEntry>& Entry : Tab.AllFiles)
			{
				if (MatchesIncludeWildcards(IncludeWildcards, Entry->Name))
				{
					Tab.FilteredFiles.Add(Entry);
					FilteredFileCount++;
					FilteredFileSize += Entry->RawSize;
				}
			}
		}

		if (Tab.ListView.IsValid())
		{
			Tab.ListView->RequestListRefresh();
		}
	}

	if (StatusText.IsValid() && !bIsLoading)
	{
		if (!SearchText.IsEmpty() && FilteredFileCount != TotalFileCount)
		{
			StatusText->SetText(FText::Format(
				LOCTEXT("StatusFormatFiltered", "Showing {0} of {1} {1}|plural(one=file,other=files) ({2} of {3})"),
				FText::AsNumber(FilteredFileCount),
				FText::AsNumber(TotalFileCount),
				FText::FromString(FormatSize(FilteredFileSize)),
				FText::FromString(FormatSize(TotalFileSize))
			));
		}
		else
		{
			StatusText->SetText(FText::Format(
				LOCTEXT("StatusFormat", "{0} {0}|plural(one=file,other=files), {1} total"),
				FText::AsNumber(TotalFileCount),
				FText::FromString(FormatSize(TotalFileSize))
			));
		}
	}

	if (MarkedStatusText.IsValid() && MarkedFileCount > 0)
	{
		MarkedStatusText->SetText(FText::Format(
			LOCTEXT("MarkedStatusFormat", "{0} {0}|plural(one=file,other=files) marked for download ({1})"),
			FText::AsNumber(MarkedFileCount),
			FText::FromString(FormatSize(MarkedFileSize))
		));
	}
}

void SBuildArtifactContentsViewer::OnSearchTextChanged(const FText& InText)
{
	SearchText = InText;
	RefreshFilteredList();
}

EColumnSortMode::Type SBuildArtifactContentsViewer::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn == ColumnId)
	{
		return SortMode;
	}
	return EColumnSortMode::None;
}

void SBuildArtifactContentsViewer::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;
	SortAndRefreshList();
}

bool SBuildArtifactContentsViewer::MatchWildcard(FStringView Wildcard, FStringView String)
{
	const TCHAR* WildcardIt  = Wildcard.GetData();
	const TCHAR* WildcardEnd = WildcardIt + Wildcard.Len();
	const TCHAR* StringIt    = String.GetData();
	const TCHAR* StringEnd   = StringIt + String.Len();

	for (; WildcardIt != WildcardEnd; WildcardIt++)
	{
		TCHAR WC = TChar<TCHAR>::ToLower(*WildcardIt);
		switch (WC)
		{
		case TEXT('?'):
			if (StringIt == StringEnd)
			{
				return false;
			}
			StringIt++;
			break;
		case TEXT('*'):
			if ((WildcardIt + 1) == WildcardEnd)
			{
				return true;
			}
			for (const TCHAR* It = StringIt; It <= StringEnd; It++)
			{
				if (MatchWildcard(FStringView(WildcardIt + 1, UE_PTRDIFF_TO_INT32(WildcardEnd - (WildcardIt + 1))),
				                  FStringView(It, UE_PTRDIFF_TO_INT32(StringEnd - It))))
				{
					return true;
				}
			}
			return false;
		default:
			if (StringIt == StringEnd)
			{
				return false;
			}
			if (TChar<TCHAR>::ToLower(*StringIt) != WC)
			{
				return false;
			}
			++StringIt;
			break;
		}
	}
	return StringIt == StringEnd;
}

void SBuildArtifactContentsViewer::ParseIncludeWildcards(const FString& FilterText, TArray<FString>& OutWildcards)
{
	TArray<FString> Tokens;
	FilterText.ParseIntoArray(Tokens, TEXT(";"), true);

	for (FString& Token : Tokens)
	{
		Token.TrimStartInline();
		Token.TrimEndInline();
		if (Token.IsEmpty())
		{
			continue;
		}

		// Lowercase and normalize separators
		Token.ToLowerInline();
		Token.ReplaceCharInline(TEXT('\\'), TEXT('/'));

		// Strip leading ./
		if (Token.StartsWith(TEXT("./")))
		{
			Token.RightChopInline(2);
		}

		if (!Token.IsEmpty())
		{
			OutWildcards.Add(MoveTemp(Token));
		}
	}
}

bool SBuildArtifactContentsViewer::MatchesIncludeWildcards(const TArray<FString>& IncludeWildcards, FStringView FileName)
{
	if (IncludeWildcards.IsEmpty())
	{
		return true;
	}

	for (const FString& Wildcard : IncludeWildcards)
	{
		if (MatchWildcard(Wildcard, FileName))
		{
			return true;
		}
	}
	return false;
}

void SBuildArtifactContentsViewer::MarkSelectedFiles()
{
	if (!PartTabs.IsValidIndex(ActiveTabIndex))
	{
		return;
	}

	FPartTab& ActiveTab = PartTabs[ActiveTabIndex];
	if (!ActiveTab.ListView.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FContentsEntry>> SelectedItems = ActiveTab.ListView->GetSelectedItems();
	for (const TSharedPtr<FContentsEntry>& Item : SelectedItems)
	{
		Item->bIsMarked = !Item->bIsMarked;
	}

	RecalculateMarkedCounts();
}

void SBuildArtifactContentsViewer::RecalculatePartMarkedCounts(FPartTab& Tab)
{
	Tab.MarkedFileCount = 0;
	Tab.MarkedFileSize = 0;
	for (const TSharedPtr<FContentsEntry>& Entry : Tab.AllFiles)
	{
		if (Entry->bIsMarked)
		{
			Tab.MarkedFileCount++;
			Tab.MarkedFileSize += Entry->RawSize;
		}
	}
}

void SBuildArtifactContentsViewer::RecalculateMarkedCounts()
{
	MarkedFileCount = 0;
	MarkedFileSize = 0;
	for (FPartTab& Tab : PartTabs)
	{
		RecalculatePartMarkedCounts(Tab);
		MarkedFileCount += Tab.MarkedFileCount;
		MarkedFileSize += Tab.MarkedFileSize;
	}

	if (MarkedStatusText.IsValid())
	{
		if (MarkedFileCount > 0)
		{
			MarkedStatusText->SetText(FText::Format(
				LOCTEXT("MarkedStatusFormat", "{0} {0}|plural(one=file,other=files) marked for download ({1})"),
				FText::AsNumber(MarkedFileCount),
				FText::FromString(FormatSize(MarkedFileSize))
			));
		}
	}

	for (FPartTab& Tab : PartTabs)
	{
		if (Tab.ListView.IsValid())
		{
			Tab.ListView->RequestListRefresh();
		}
	}
}

void SBuildArtifactContentsViewer::MarkAllFiles()
{
	for (FPartTab& Tab : PartTabs)
	{
		for (const TSharedPtr<FContentsEntry>& Entry : Tab.AllFiles)
		{
			Entry->bIsMarked = true;
		}
	}

	RecalculateMarkedCounts();
}

void SBuildArtifactContentsViewer::ClearAllMarkedFiles()
{
	for (FPartTab& Tab : PartTabs)
	{
		for (const TSharedPtr<FContentsEntry>& Entry : Tab.AllFiles)
		{
			Entry->bIsMarked = false;
		}
	}

	MarkedFileCount = 0;
	MarkedFileSize = 0;

	for (FPartTab& Tab : PartTabs)
	{
		Tab.MarkedFileCount = 0;
		Tab.MarkedFileSize = 0;
		if (Tab.ListView.IsValid())
		{
			Tab.ListView->RequestListRefresh();
		}
	}
}

FReply SBuildArtifactContentsViewer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::SpaceBar)
	{
		MarkSelectedFiles();
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FDownloadSpec SBuildArtifactContentsViewer::ComposeDownloadSpec() const
{
	FDownloadSpec Result;

	struct FPartInfo
	{
		FCbObjectId PartId;
		TArray<FString> Files;
	};

	TArray<TPair<FString, FPartInfo>> PartialParts;

	for (const FPartTab& Tab : PartTabs)
	{
		if (Tab.MarkedFileCount == 0)
		{
			continue;
		}

		if (Tab.IsFullyMarked())
		{
			Result.FullyMarkedPartNames.Add(Tab.PartName);
		}
		else
		{
			FPartInfo Info;
			Info.PartId = Tab.PartId;
			for (const TSharedPtr<FContentsEntry>& Entry : Tab.AllFiles)
			{
				if (Entry->bIsMarked)
				{
					Info.Files.Add(Entry->Name);
				}
			}
			PartialParts.Emplace(Tab.PartName, MoveTemp(Info));
		}
	}

	// Build JSON only for partially-marked parts; skip entirely when all
	// marked parts are fully marked since PartNames alone is sufficient.
	if (!PartialParts.IsEmpty())
	{
		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);

		Writer->WriteObjectStart();
		Writer->WriteObjectStart(TEXT("parts"));
		for (const TPair<FString, FPartInfo>& Pair : PartialParts)
		{
			Writer->WriteObjectStart(Pair.Key);
			if (!(Pair.Value.PartId == FCbObjectId()))
			{
				Writer->WriteValue(TEXT("partId"), FString(WriteToString<64>(Pair.Value.PartId)));
			}
			Writer->WriteArrayStart(TEXT("files"));
			for (const FString& FileName : Pair.Value.Files)
			{
				Writer->WriteValue(FileName);
			}
			Writer->WriteArrayEnd();
			Writer->WriteObjectEnd();
		}
		Writer->WriteObjectEnd();
		Writer->WriteObjectEnd();

		Writer->Close();
		Result.SpecJSON = MoveTemp(OutputString);
	}

	return Result;
}

FString SBuildArtifactContentsViewer::ComposeDownloadSpecJSON() const
{
	return ComposeDownloadSpec().SpecJSON;
}

TSharedPtr<SWidget> SBuildArtifactContentsViewer::OnContextMenuOpening()
{
	if (!PartTabs.IsValidIndex(ActiveTabIndex))
	{
		return nullptr;
	}

	FPartTab& ActiveTab = PartTabs[ActiveTabIndex];
	if (!ActiveTab.ListView.IsValid())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FContentsEntry>> SelectedItems = ActiveTab.ListView->GetSelectedItems();

	// All entries are file entries now
	TArray<TSharedPtr<FContentsEntry>> FileEntries = MoveTemp(SelectedItems);
	if (FileEntries.IsEmpty() || !OnStartFileDownload.IsBound())
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(/*bCloseAfterSelection*/ true, nullptr, TSharedPtr<FExtender>(),
		/*bCloseSelfOnly*/ false, &FCoreStyle::Get(), /*bSearchable*/ false);

	FText Label = FileEntries.Num() == 1
		? LOCTEXT("OpenFile", "Open")
		: FText::Format(LOCTEXT("OpenFiles", "Open {0} Files"), FText::AsNumber(FileEntries.Num()));
	FText Tooltip = FileEntries.Num() == 1
		? LOCTEXT("OpenFileTooltip", "Download this file and open it with the default application")
		: LOCTEXT("OpenFilesTooltip", "Download selected files and open them with the default application");

	FText ExploreLabel = FileEntries.Num() == 1
		? LOCTEXT("ExploreFile", "Open in File Browser")
		: FText::Format(LOCTEXT("ExploreFiles", "Open {0} Files in File Browser"), FText::AsNumber(FileEntries.Num()));
	FText ExploreTooltip = FileEntries.Num() == 1
		? LOCTEXT("ExploreFileTooltip", "Download this file and open its folder in the file browser")
		: LOCTEXT("ExploreFilesTooltip", "Download selected files and open their folders in the file browser");

	// Capture a copy for the explore entry before moving FileEntries into the open entry
	TArray<TSharedPtr<FContentsEntry>> ExploreEntries = FileEntries;

	MenuBuilder.AddMenuEntry(
		Label,
		Tooltip,
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this, Entries = MoveTemp(FileEntries)]()
		{
			// Must copy because FUIAction stores a shared const copy of the delegate
			TArray<TSharedPtr<FContentsEntry>> EntriesCopy = Entries;
			OpenFileEntries(MoveTemp(EntriesCopy));
		}))
	);

	MenuBuilder.AddMenuEntry(
		ExploreLabel,
		ExploreTooltip,
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this, Entries = MoveTemp(ExploreEntries)]()
		{
			TArray<TSharedPtr<FContentsEntry>> EntriesCopy = Entries;
			OpenFileEntries(MoveTemp(EntriesCopy), EFileOpenAction::ExploreFolder);
		}))
	);

	return MenuBuilder.MakeWidget();
}

void SBuildArtifactContentsViewer::OpenFileEntries(TArray<TSharedPtr<FContentsEntry>>&& FileEntries, EFileOpenAction Action)
{
	FString Spec = ComposeFileDownloadSpec(FileEntries);
	UE::Zen::Build::FBuildServiceInstance::FBuildTransfer Transfer = OnStartFileDownload.Execute(MoveTemp(Spec));

	for (const TSharedPtr<FContentsEntry>& Entry : FileEntries)
	{
		Entry->bIsDownloading = true;
	}

	bool bNeedTimer = PendingFileOpens.IsEmpty();
	PendingFileOpens.Add({ MoveTemp(Transfer), MoveTemp(FileEntries), Action });

	if (bNeedTimer)
	{
		RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SBuildArtifactContentsViewer::PollPendingFileOpens));
	}
}

FString SBuildArtifactContentsViewer::ComposeFileDownloadSpec(const TArray<TSharedPtr<FContentsEntry>>& FileEntries) const
{
	struct FPartInfo
	{
		FCbObjectId PartId;
		TArray<FString> Files;
	};

	TArray<TPair<FString, FPartInfo>> Parts;

	for (const TSharedPtr<FContentsEntry>& FileEntry : FileEntries)
	{
		FString PartNameStr(FileEntry->PartName);

		// Find or create the part group
		FPartInfo* Found = nullptr;
		for (TPair<FString, FPartInfo>& Pair : Parts)
		{
			if (Pair.Key == PartNameStr)
			{
				Found = &Pair.Value;
				break;
			}
		}
		if (!Found)
		{
			Parts.Emplace(PartNameStr, FPartInfo{ FileEntry->PartId, {} });
			Found = &Parts.Last().Value;
		}
		Found->Files.Add(FileEntry->Name);
	}

	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);

	Writer->WriteObjectStart();
	Writer->WriteObjectStart(TEXT("parts"));
	for (const TPair<FString, FPartInfo>& Pair : Parts)
	{
		Writer->WriteObjectStart(Pair.Key);
		if (!(Pair.Value.PartId == FCbObjectId()))
		{
			Writer->WriteValue(TEXT("partId"), FString(WriteToString<64>(Pair.Value.PartId)));
		}
		Writer->WriteArrayStart(TEXT("files"));
		for (const FString& FileName : Pair.Value.Files)
		{
			Writer->WriteValue(FileName);
		}
		Writer->WriteArrayEnd();
		Writer->WriteObjectEnd();
	}
	Writer->WriteObjectEnd();
	Writer->WriteObjectEnd();

	Writer->Close();
	return OutputString;
}

EActiveTimerReturnType SBuildArtifactContentsViewer::PollPendingFileOpens(double InCurrentTime, float InDeltaTime)
{
	using namespace UE::Zen::Build;

	for (int32 i = PendingFileOpens.Num() - 1; i >= 0; --i)
	{
		FPendingFileOpen& Pending = PendingFileOpens[i];
		FBuildServiceInstance::EBuildTransferStatus Status = Pending.Transfer.GetStatus();

		if (Status == FBuildServiceInstance::EBuildTransferStatus::Succeeded)
		{
			switch (Pending.Action)
			{
			case EFileOpenAction::LaunchFile:
				for (const TSharedPtr<FContentsEntry>& Entry : Pending.Entries)
				{
					FString FullPath = FPaths::Combine(Pending.Transfer.GetDestination(), Entry->Name);
					FPlatformProcess::LaunchFileInDefaultExternalApplication(*FullPath, nullptr, ELaunchVerb::Edit);
					Entry->bIsDownloading = false;
				}
				break;

			case EFileOpenAction::ExploreFolder:
				{
					FString CommonPath;
					for (const TSharedPtr<FContentsEntry>& Entry : Pending.Entries)
					{
						FString Folder = FPaths::GetPath(FPaths::Combine(Pending.Transfer.GetDestination(), Entry->Name));
						if (CommonPath.IsEmpty())
						{
							CommonPath = Folder;
						}
						else if (!FPaths::IsSamePath(CommonPath, Folder))
						{
							while (!FPaths::IsUnderDirectory(Folder, CommonPath))
							{
								CommonPath = FPaths::GetPath(CommonPath);
							}
						}
						Entry->bIsDownloading = false;
					}
					if (!CommonPath.IsEmpty())
					{
						FPlatformProcess::ExploreFolder(*CommonPath);
					}
				}
				break;
			}
			PendingFileOpens.RemoveAt(i);
		}
		else if (Status == FBuildServiceInstance::EBuildTransferStatus::Failed ||
		         Status == FBuildServiceInstance::EBuildTransferStatus::Canceled)
		{
			for (const TSharedPtr<FContentsEntry>& Entry : Pending.Entries)
			{
				Entry->bIsDownloading = false;
			}
			PendingFileOpens.RemoveAt(i);
		}
	}

	return PendingFileOpens.IsEmpty() ? EActiveTimerReturnType::Stop : EActiveTimerReturnType::Continue;
}

#undef LOCTEXT_NAMESPACE
