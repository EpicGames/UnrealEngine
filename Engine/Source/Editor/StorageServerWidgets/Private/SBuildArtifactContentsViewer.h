// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Experimental/BuildServerInterface.h"

struct FContentsEntry
{
	bool bIsMarked = false;     // Persistent download mark
	bool bIsDownloading = false; // true while an "Open" download is in progress
	FString Name;               // File name
	FStringView PartName;       // Non-owning view into FPartTab::PartName (stable after SetContents)
	FCbObjectId PartId;         // Parent part ID
	uint64 RawSize = 0;
};

struct FPartTab
{
	FString PartName;
	FCbObjectId PartId;
	TArray<TSharedPtr<FContentsEntry>> AllFiles;
	TArray<TSharedPtr<FContentsEntry>> FilteredFiles;
	TSharedPtr<SListView<TSharedPtr<FContentsEntry>>> ListView;
	int32 TotalFileCount = 0;
	uint64 TotalFileSize = 0;
	int32 MarkedFileCount = 0;
	uint64 MarkedFileSize = 0;

	ECheckBoxState GetCheckState() const;
	bool IsFullyMarked() const;
};

struct FDownloadSpec
{
	FString SpecJSON;                        // JSON for partially-marked parts (individual files)
	TArray<FString> FullyMarkedPartNames;    // Parts where every file is marked
};

/**
 * Row widget for a single contents entry, using SMultiColumnTableRow so columns
 * track header resizing automatically.
 */
class SContentsEntryRow : public SMultiColumnTableRow<TSharedPtr<FContentsEntry>>
{
public:
	SLATE_BEGIN_ARGS(SContentsEntryRow) {}
		SLATE_EVENT(FSimpleDelegate, OnMarkChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FContentsEntry> InEntry);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<FContentsEntry> Entry;
	FSimpleDelegate OnMarkChanged;
};

DECLARE_DELEGATE_RetVal_OneParam(
	UE::Zen::Build::FBuildServiceInstance::FBuildTransfer,  // return: transfer handle
	FOnStartFileDownload,
	FString&&  /* DownloadSpecJSON */
);

/**
 * Widget for viewing build artifact file contents in a tabbed view grouped by build part
 */
class SBuildArtifactContentsViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBuildArtifactContentsViewer)
	{}
		SLATE_ARGUMENT(FString, ArtifactName)
		SLATE_EVENT(FOnStartFileDownload, OnStartFileDownload)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SBuildArtifactContentsViewer();

	/** Thread-safe: queues contents from a background thread for consumption on the Slate thread. */
	void QueueContents(TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>&& BuildContents);

	/** Toggle bIsMarked on selected items in the active tab and update counts. */
	void MarkSelectedFiles();

	/** Mark all file entries for download across all tabs. */
	void MarkAllFiles();

	/** Unmark all marked files across all tabs. */
	void ClearAllMarkedFiles();

	bool HasMarkedFiles() const { return MarkedFileCount > 0; }
	int32 GetMarkedFileCount() const { return MarkedFileCount; }
	uint64 GetMarkedFileSize() const { return MarkedFileSize; }

	/** Recompute MarkedFileCount/MarkedFileSize from all PartTabs and update status text. */
	void RecalculateMarkedCounts();

	/** Build download spec from marked entries, separating fully-marked parts. */
	FDownloadSpec ComposeDownloadSpec() const;

	/** Build download spec JSON from marked entries (thin wrapper). */
	FString ComposeDownloadSpecJSON() const;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	static FString FormatSize(uint64 SizeInBytes);

private:
	void SetContents(TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>& BuildContents);

	EActiveTimerReturnType PollPendingContents(double InCurrentTime, float InDeltaTime);

	enum class EFileOpenAction : uint8
	{
		LaunchFile,     // Open each file in its default application
		ExploreFolder,  // Open the containing folder(s) in the native file browser
	};

	TSharedPtr<SWidget> OnContextMenuOpening();
	void OpenFileEntries(TArray<TSharedPtr<FContentsEntry>>&& FileEntries, EFileOpenAction Action = EFileOpenAction::LaunchFile);
	FString ComposeFileDownloadSpec(const TArray<TSharedPtr<FContentsEntry>>& FileEntries) const;
	EActiveTimerReturnType PollPendingFileOpens(double InCurrentTime, float InDeltaTime);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FContentsEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	void SetActiveTab(int32 Index);
	void RecalculatePartMarkedCounts(FPartTab& Tab);
	void RebuildTabBar();
	void RefreshFilteredList();
	void SortAndRefreshList();

	void OnSearchTextChanged(const FText& InText);

	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	static bool MatchWildcard(FStringView Wildcard, FStringView String);
	static void ParseIncludeWildcards(const FString& FilterText, TArray<FString>& OutWildcards);
	static bool MatchesIncludeWildcards(const TArray<FString>& IncludeWildcards, FStringView FileName);

	FCriticalSection PendingContentsMutex;
	TOptional<TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>> PendingContents;

	TArray<FPartTab> PartTabs;

	TSharedPtr<SWidgetSwitcher> ContentSwitcher;
	TSharedPtr<SScrollBox> TabBarScrollBox;
	int32 ActiveTabIndex = 0;

	TSharedPtr<STextBlock> StatusText;
	TSharedPtr<STextBlock> MarkedStatusText;

	FText SearchText;

	FName SortByColumn;
	EColumnSortMode::Type SortMode = EColumnSortMode::None;

	int32 TotalFileCount = 0;
	uint64 TotalFileSize = 0;

	int32 MarkedFileCount = 0;
	uint64 MarkedFileSize = 0;

	struct FPendingFileOpen
	{
		UE::Zen::Build::FBuildServiceInstance::FBuildTransfer Transfer;
		TArray<TSharedPtr<FContentsEntry>> Entries;
		EFileOpenAction Action = EFileOpenAction::LaunchFile;
	};

	TArray<FPendingFileOpen> PendingFileOpens;
	FOnStartFileDownload OnStartFileDownload;

	bool bIsLoading = true;

	FButtonStyle InactiveTabButtonStyle;
	FButtonStyle ActiveTabButtonStyle;
};
