// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildArtifactMetadataViewer.h"
#include "Internationalization/Internationalization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SSearchBox.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "BuildArtifactMetadata"

void SBuildArtifactMetadataViewer::Construct(const FArguments& InArgs)
{
	ArtifactName = InArgs._ArtifactName;

	// Parse the metadata
	ParseMetadata(InArgs._BuildId, InArgs._Metadata);

	// Initialize filtered list with all entries
	RefreshFilteredList();

	ChildSlot
	[
		SNew(SVerticalBox)

		// Header
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f)
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("MetadataHeader", "Metadata for: {0}"), FText::FromString(ArtifactName)))
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
		]

		// Search box
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(SSearchBox)
			.HintText(LOCTEXT("SearchHint", "Search metadata..."))
			.OnTextChanged(this, &SBuildArtifactMetadataViewer::OnSearchTextChanged)
		]

		// Metadata list
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(4.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FMetadataEntry>>)
				.ListItemsSource(&FilteredMetadataEntries)
				.OnGenerateRow(this, &SBuildArtifactMetadataViewer::OnGenerateRow)
				.OnContextMenuOpening(this, &SBuildArtifactMetadataViewer::OnContextMenuOpening)
				.SelectionMode(ESelectionMode::Single)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Key")
					.DefaultLabel(LOCTEXT("KeyColumn", "Key"))
					.FillWidth(0.3f)

					+ SHeaderRow::Column("Value")
					.DefaultLabel(LOCTEXT("ValueColumn", "Value"))
					.FillWidth(0.6f)

					+ SHeaderRow::Column("Type")
					.DefaultLabel(LOCTEXT("TypeColumn", "Type"))
					.FillWidth(0.1f)
				)
			]
		]
	];
}

TSharedRef<ITableRow> SBuildArtifactMetadataViewer::OnGenerateRow(TSharedPtr<FMetadataEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FMetadataEntry>>, OwnerTable)
		[
			SNew(SHorizontalBox)

			// Key column
			+ SHorizontalBox::Slot()
			.FillWidth(0.3f)
			.Padding(4.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Entry->Key))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
				.ColorAndOpacity(FStyleColors::AccentGreen)
			]

			// Value column
			+ SHorizontalBox::Slot()
			.FillWidth(0.6f)
			.Padding(4.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Entry->Value))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
				.AutoWrapText(true)
			]

			// Type column
			+ SHorizontalBox::Slot()
			.FillWidth(0.1f)
			.Padding(4.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Entry->Type))
				.Font(FCoreStyle::GetDefaultFontStyle("Italic", 8))
				.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor() * 0.6f)
			]
		];
}

void SBuildArtifactMetadataViewer::ParseMetadata(const FCbObjectId& BuildId, const FCbObject& Metadata)
{
	MetadataEntries.Empty();

	// Add the build id as the first entry
	MetadataEntries.Add(MakeShared<FMetadataEntry>(TEXT("buildId"), FString(WriteToString<64>(BuildId)), TEXT("ObjectId")));

	// Iterate through all fields in the compact binary object
	for (FCbFieldView Field : Metadata)
	{
		FUtf8StringView FieldName = Field.GetName();
		FString Key = FString(FieldName.Len(), FieldName.GetData());
		FString Value;
		FString Type;

		// Extract value based on field type
		// Check more specific types before more general ones to avoid misidentification

		// Check null first (most specific)
		if (Field.IsNull())
		{
			Value = TEXT("null");
			Type = TEXT("Null");
		}
		// Check bool before integers
		else if (Field.IsBool())
		{
			Value = Field.AsBool() ? TEXT("true") : TEXT("false");
			Type = TEXT("Bool");
		}
		// Check DateTime before integer (DateTime is stored as integer internally)
		else if (Field.IsDateTime())
		{
			FDateTime DateTime = Field.AsDateTime();
			Value = DateTime.ToString();
			Type = TEXT("DateTime");
		}
		// Check ObjectId before binary (ObjectId is a specific binary type)
		else if (Field.IsObjectId())
		{
			Value = FString(WriteToString<64>(Field.AsObjectId()));
			Type = TEXT("ObjectId");
		}
		// Check UUID before binary (UUID is a specific binary type)
		else if (Field.IsUuid())
		{
			Value = FString(WriteToString<48>(Field.AsUuid()));
			Type = TEXT("UUID");
		}
		// Check Hash before binary (Hash is a specific binary type)
		else if (Field.IsHash())
		{
			Value = FString(WriteToString<48>(Field.AsHash()));
			Type = TEXT("Hash");
		}
		// Check integer before float (integer is more specific than float)
		else if (Field.IsInteger())
		{
			Value = FString::Printf(TEXT("%lld"), Field.AsInt64());
			Type = TEXT("Integer");
		}
		else if (Field.IsFloat())
		{
			const FCulturePtr LocaleCulture = FInternationalization::Get().GetCulture(FPlatformMisc::GetDefaultLocale());
			const FNumberFormattingOptions Options = FNumberFormattingOptions()
				.SetUseGrouping(false);
			Value = FText::AsNumber(Field.AsDouble(), &Options, LocaleCulture).ToString();
			Type = TEXT("Float");
		}
		// Check string
		else if (Field.IsString())
		{
			Value = FString(FUTF8ToTCHAR(Field.AsString()));
			Type = TEXT("String");
		}
		// Check generic binary after specific binary types
		else if (Field.IsBinary())
		{
			FMemoryView BinaryView = Field.AsBinaryView();
			Value = FString::Printf(TEXT("[%lld bytes]"), BinaryView.GetSize());
			Type = TEXT("Binary");
		}
		// Check container types
		else if (Field.IsArray())
		{
			Value = FString::Printf(TEXT("[%lld elements]"), Field.AsArrayView().Num());
			Type = TEXT("Array");
		}
		else if (Field.IsObject())
		{
			int32 FieldCount = 0;
			for (FCbFieldView ObjectField : Field)
			{
				++FieldCount;
			}
			Value = FString::Printf(TEXT("{%d fields}"), FieldCount);
			Type = TEXT("Object");
		}
		else
		{
			Value = TEXT("<Unknown>");
			Type = TEXT("Unknown");
		}

		MetadataEntries.Add(MakeShared<FMetadataEntry>(Key, Value, Type));
	}

	// Sort entries alphabetically by key
	MetadataEntries.Sort([](const TSharedPtr<FMetadataEntry>& A, const TSharedPtr<FMetadataEntry>& B)
	{
		return A->Key < B->Key;
	});
}

void SBuildArtifactMetadataViewer::RefreshFilteredList()
{
	FilteredMetadataEntries.Empty();

	if (SearchText.IsEmpty())
	{
		// No filter, show all entries
		FilteredMetadataEntries = MetadataEntries;
	}
	else
	{
		// Filter entries based on search text (case-insensitive)
		FString SearchString = SearchText.ToString().ToLower();
		for (const TSharedPtr<FMetadataEntry>& Entry : MetadataEntries)
		{
			if (Entry->Key.ToLower().Contains(SearchString) ||
				Entry->Value.ToLower().Contains(SearchString) ||
				Entry->Type.ToLower().Contains(SearchString))
			{
				FilteredMetadataEntries.Add(Entry);
			}
		}
	}

	// Refresh the list view
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

void SBuildArtifactMetadataViewer::OnSearchTextChanged(const FText& InText)
{
	SearchText = InText;
	RefreshFilteredList();
}

TSharedPtr<SWidget> SBuildArtifactMetadataViewer::OnContextMenuOpening()
{
	TArray<TSharedPtr<FMetadataEntry>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return nullptr;
	}

	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	MenuBuilder.BeginSection("MetadataCopy", LOCTEXT("CopySection", "Copy"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyKey", "Copy Key"),
			LOCTEXT("CopyKeyTooltip", "Copy the metadata key to clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SBuildArtifactMetadataViewer::CopyKeyToClipboard))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyValue", "Copy Value"),
			LOCTEXT("CopyValueTooltip", "Copy the metadata value to clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SBuildArtifactMetadataViewer::CopyValueToClipboard))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyEntry", "Copy Key: Value"),
			LOCTEXT("CopyEntryTooltip", "Copy the entire entry as 'Key: Value' to clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SBuildArtifactMetadataViewer::CopyEntryToClipboard))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SBuildArtifactMetadataViewer::CopyKeyToClipboard()
{
	TArray<TSharedPtr<FMetadataEntry>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(*SelectedItems[0]->Key);
	}
}

void SBuildArtifactMetadataViewer::CopyValueToClipboard()
{
	TArray<TSharedPtr<FMetadataEntry>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(*SelectedItems[0]->Value);
	}
}

void SBuildArtifactMetadataViewer::CopyEntryToClipboard()
{
	TArray<TSharedPtr<FMetadataEntry>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		FString EntryText = FString::Printf(TEXT("%s: %s"), *SelectedItems[0]->Key, *SelectedItems[0]->Value);
		FPlatformApplicationMisc::ClipboardCopy(*EntryText);
	}
}

#undef LOCTEXT_NAMESPACE
