// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Serialization/CompactBinary.h"

/**
 * Widget for displaying build artifact metadata in a property grid style
 */
class SBuildArtifactMetadataViewer : public SCompoundWidget
{
public:
	/** Struct to hold a metadata key-value pair for display */
	struct FMetadataEntry
	{
		FString Key;
		FString Value;
		FString Type;

		FMetadataEntry(const FString& InKey, const FString& InValue, const FString& InType)
			: Key(InKey), Value(InValue), Type(InType)
		{}
	};

	SLATE_BEGIN_ARGS(SBuildArtifactMetadataViewer)
	{}
		SLATE_ARGUMENT(FCbObject, Metadata)
		SLATE_ARGUMENT(FString, ArtifactName)
		SLATE_ARGUMENT(FCbObjectId, BuildId)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Generate a row widget for a metadata entry */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FMetadataEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Generate context menu for a row */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Parse FCbObject metadata into display entries */
	void ParseMetadata(const FCbObjectId& BuildId, const FCbObject& Metadata);

	/** Refresh the filtered list based on search text */
	void RefreshFilteredList();

	/** Called when search text changes */
	void OnSearchTextChanged(const FText& InText);

	/** Copy selected entry key to clipboard */
	void CopyKeyToClipboard();

	/** Copy selected entry value to clipboard */
	void CopyValueToClipboard();

	/** Copy selected entry as "Key: Value" to clipboard */
	void CopyEntryToClipboard();

	/** List of all metadata entries */
	TArray<TSharedPtr<FMetadataEntry>> MetadataEntries;

	/** Filtered list of metadata entries based on search */
	TArray<TSharedPtr<FMetadataEntry>> FilteredMetadataEntries;

	/** List view widget */
	TSharedPtr<SListView<TSharedPtr<FMetadataEntry>>> ListView;

	/** Current search filter text */
	FText SearchText;

	/** Artifact name for display */
	FString ArtifactName;
};
