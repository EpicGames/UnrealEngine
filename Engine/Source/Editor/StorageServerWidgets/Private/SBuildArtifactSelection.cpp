// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildArtifactSelection.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/CoreDelegates.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/StyleColors.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "ZenServiceInstanceManager.h"
#include "SBuildArtifactMetadataViewer.h"
#include "SBuildArtifactContentsViewer.h"
#include "Dialog/DialogCommands.h"
#include "Dialog/SCustomDialog.h"


namespace
{
	FString GetOpenFileTempRoot()
	{
		static FString TempRoot = FPaths::Combine(
			FPlatformProcess::UserTempDir(),
			TEXT("BuildStorageTool"),
			TEXT("Scratch"));
		static bool bCleanupRegistered = false;
		if (!bCleanupRegistered)
		{
			bCleanupRegistered = true;
			FCoreDelegates::OnPreExit.AddStatic([]()
			{
				IFileManager::Get().DeleteDirectory(*GetOpenFileTempRoot(), false, true);
			});
		}
		return TempRoot;
	}
}

#define LOCTEXT_NAMESPACE "StorageServerBuild"

namespace UE::BuildArtifactSelection::Internal
{
	namespace FBuildArtifactIds
	{
		const FName ColSelected = TEXT("Selected");
		const FName ColName = TEXT("Name");
	}
}

void SBuildArtifactSelection::Construct(const FArguments& InArgs)
{
	using namespace UE::BuildSelection::Internal;
	BuildServiceInstance = InArgs._BuildServiceInstance;
	OnDownloadWithSpec = InArgs._OnDownloadWithSpec;

	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	if (FString PastConfiguration; ReadSetting(TEXT("Configuration"), PastConfiguration))
	{
		SelectedConfiguration = MakeShared<FString>(MoveTemp(PastConfiguration));
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
					.ColorAndOpacity(TitleColor)
					.Font(TitleFont)
					.Text(LOCTEXT("BuildArtifactSelection_AvailableBuilds", "Available Builds"))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 0, 0)
			[
				SAssignNew(SelectedGroupArtifactHeaderGrid, SGridPanel)
			]
		]

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SAssignNew(SelectedGroupArtifactGrid, SGridPanel)
		]
	];
}

bool SBuildArtifactSelection::HasArtifactConfigurations() const
{
	return !ArtifactConfigurationSet.IsEmpty();
}

TSharedPtr<FString> SBuildArtifactSelection::GetSelectedConfiguration() const
{
	return SelectedConfiguration;
}

const TArray<FString>& SBuildArtifactSelection::GetSelectedArtifacts() const
{
	return SelectedArtifacts;
}

void SBuildArtifactSelection::SetAvailableConfigurations(
	TSet<FString>&& InArtifactConfigurationSet,
	TMultiMap<FString, FString>&& InArtifactNameMap
)
{
	using namespace UE::BuildSelection::Internal;
	ArtifactConfigurationSet = MoveTemp(InArtifactConfigurationSet);
	ArtifactNameMap = MoveTemp(InArtifactNameMap);

	ArtifactConfigurationSet.StableSort(TLess());
	ArtifactNameMap.ValueStableSort(TLess());

	ArtifactConfigurationList.Empty();
	if (ArtifactConfigurationSet.Num() > 1)
	{
		ArtifactConfigurationList.Add(MakeShared<FString>(TEXT("All")));
	}
	for (const FString& ArtifactConfiguration : ArtifactConfigurationSet)
	{
		ArtifactConfigurationList.Add(MakeShared<FString>(ArtifactConfiguration));
	}
	if (!ArtifactConfigurationList.IsEmpty())
	{
		ConformSelection(SelectedConfiguration, ArtifactConfigurationList);
	}

	if (!ArtifactConfigurationList.IsEmpty())
	{
		WriteSetting(TEXT("Configuration"), SelectedConfiguration ? **SelectedConfiguration : TEXT(""));
	}
}

void SBuildArtifactSelection::Refresh(const FStringView InNamespace, const FStringView InGroupDisplayName, const TMap<FString, UE::BuildSelection::Internal::FArtifact>* InNamedArtifacts, FCbObjectId InHighlightBuildId, bool bInCanViewContents)
{
	using namespace UE::BuildSelection::Internal;

	Namespace = InNamespace;
	bCanViewContents = bInCanViewContents;
	HighlightedBuildId = InHighlightBuildId;
	GroupDisplayName = InGroupDisplayName;
	if (InNamedArtifacts)
	{
		NamedArtifacts = *InNamedArtifacts;
	}
	else
	{
		NamedArtifacts.Empty();
	}

	RebuildSelectedGroupArtifactHeader();
	RebuildSelectedGroupArtifactContents();
}

void SBuildArtifactSelection::RebuildSelectedGroupArtifactHeader()
{
	if (!SelectedGroupArtifactHeaderGrid)
	{
		return;
	}

	SelectedGroupArtifactHeaderGrid->ClearChildren();
	if (ArtifactConfigurationSet.IsEmpty())
	{
		return;
	}

	SelectedGroupArtifactHeaderGrid->AddSlot(0, 0)
		.Padding(10,0,2,0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Justification(ETextJustify::Right)
			.Text(LOCTEXT("BuildSelection_Configuration", "Configuration:"))
		];

	SelectedGroupArtifactHeaderGrid->AddSlot(1, 0)
		.Padding(0,0,0,0)
		.VAlign(VAlign_Center)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&ArtifactConfigurationList)
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewSelectedConfiguration, ESelectInfo::Type SelectInfo)
			{
				if (NewSelectedConfiguration.IsValid() && *NewSelectedConfiguration != *SelectedConfiguration)
				{
					SetSelectedConfiguration(NewSelectedConfiguration);
					RebuildSelectedGroupArtifactContents();
				}
			})
			.OnGenerateWidget_Lambda([this](TSharedPtr<FString> Item)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Item));
				})
			[
				SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return FText::FromString(SelectedConfiguration ? **SelectedConfiguration : TEXT(""));
					})
			]
		];
}

void SBuildArtifactSelection::RebuildSelectedGroupArtifactContents()
{
	using namespace UE::BuildSelection::Internal;

	if (!SelectedGroupArtifactGrid)
	{
		return;
	}

	SelectedGroupArtifactGrid->ClearChildren();

	int32 Row = 0;
	int32 Column = 0;
	TArray<FString> ArtifactNames;
	const bool bPresentConsistentArtifactNameSet = false;
	if (bPresentConsistentArtifactNameSet)
	{
		if (!SelectedConfiguration || ArtifactConfigurationSet.IsEmpty())
		{
			ArtifactNameMap.GenerateValueArray(ArtifactNames);
		}
		else if (*SelectedConfiguration == TEXT("All"))
		{
			for (const TPair<FString, FString>& ArtifactNamePair : ArtifactNameMap)
			{
				ArtifactNames.AddUnique(ArtifactNamePair.Value);
			}
			ArtifactNames.StableSort(TLess());
		}
		else
		{
			ArtifactNameMap.MultiFind(*SelectedConfiguration, ArtifactNames);
		}
	}
	else
	{
		if (!SelectedConfiguration || ArtifactConfigurationSet.IsEmpty() || *SelectedConfiguration == TEXT("All"))
		{
			NamedArtifacts.GenerateKeyArray(ArtifactNames);
		}
		else
		{
			FName SelectedConfigurationName(*SelectedConfiguration);
			for (const TPair<FString, FArtifact>& NameArtifactPair : NamedArtifacts)
			{
				if ((NameArtifactPair.Value.BuildRecord.BuildId == HighlightedBuildId) ||
					NameArtifactPair.Value.Configurations.Contains(SelectedConfigurationName))
				{
					ArtifactNames.Add(NameArtifactPair.Key);
				}
			}
		}
		ArtifactNames.StableSort(TLess());
	}

	const int32 TotalListCount = ArtifactNames.Num();
	// Heuristic for choosing grid rows when laying out rows and columns for the quantity of platforms
	const int32 MaxRows = FMath::Max(4, FMath::FloorToInt32(FMath::Pow(float(TotalListCount), 0.7)));

	auto IsBuildGroupArtifactEnabled = [this] (const FString& ArtifactName)
	{
		const FArtifact* Artifact = NamedArtifacts.Find(*ArtifactName);
		if (!Artifact)
		{
			return false;
		}
		if (Artifact->BuildRecord.BuildId == HighlightedBuildId)
		{
			return true;
		}
		if (ArtifactConfigurationSet.IsEmpty() || !SelectedConfiguration || *SelectedConfiguration == TEXT("All"))
		{
			return true;
		}
		if (*SelectedConfiguration == TEXT("None"))
		{
			return Artifact->Configurations.IsEmpty();
		}
		return Artifact->Configurations.Contains(FName(*SelectedConfiguration));
	};
	const FSlateFontInfo RegularFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	const FSlateFontInfo HighlightFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	for (const FString& ArtifactName : ArtifactNames)
	{
		TSharedPtr<SCheckBox> CurrentCheckbox;
		SelectedGroupArtifactGrid->AddSlot(Column, Row)
			.Padding(0,0,0,2)
			[
				SNew(SHorizontalBox)
				.IsEnabled_Lambda([this, ArtifactName, IsBuildGroupArtifactEnabled]
				{
					return IsBuildGroupArtifactEnabled(ArtifactName);
				})
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SAssignNew(CurrentCheckbox, SCheckBox)
					.IsChecked_Lambda([this, ArtifactName, IsBuildGroupArtifactEnabled]
					{
						bool bIsChecked = IsBuildGroupArtifactEnabled(ArtifactName) && SelectedArtifacts.Contains(ArtifactName);
						return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this, ArtifactName](ECheckBoxState InNewState)
					{
						if (InNewState == ECheckBoxState::Checked)
						{
							SelectedArtifacts.AddUnique(ArtifactName);
						}
						else if (InNewState == ECheckBoxState::Unchecked)
						{
							SelectedArtifacts.Remove(ArtifactName);
						}
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FStyleDefaults::GetNoBrush())
					.Padding(0)
					.OnMouseButtonDown_Lambda([this, ArtifactName](const FGeometry&, const FPointerEvent& MouseEvent) -> FReply
					{
						if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
						{
							TSharedRef<SWidget> MenuContent = CreateArtifactMenuWidget(NamedArtifacts.Find(ArtifactName), ArtifactName);
							FSlateApplication::Get().PushMenu(
								SharedThis(this),
								FWidgetPath(),
								MenuContent,
								FSlateApplication::Get().GetCursorPos(),
								FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
							);
							return FReply::Handled();
						}
						return FReply::Unhandled();
					})
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
						.IsFocusable(false)

						.OnClicked_Lambda([this, CurrentCheckbox, ArtifactName]()
							{
								ECheckBoxState NewState = ECheckBoxState::Checked;
								if (CurrentCheckbox->IsChecked())
								{
									NewState = ECheckBoxState::Unchecked;
								}
								CurrentCheckbox->SetIsChecked(NewState);
								if (NewState == ECheckBoxState::Checked)
								{
									SelectedArtifacts.AddUnique(ArtifactName);
								}
								else if (NewState == ECheckBoxState::Unchecked)
								{
									SelectedArtifacts.Remove(ArtifactName);
								}
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Left)
							.Text(FText::FromString(ArtifactName))
							.ToolTipText_Lambda([this, ArtifactName]() -> FText
							{
								if (const FArtifact* Artifact = NamedArtifacts.Find(ArtifactName))
								{
									FCbFieldView TotalSizeField = Artifact->BuildRecord.Metadata["totalSize"];
									if (TotalSizeField.HasValue() && !TotalSizeField.HasError())
									{
										uint64 TotalSize = 0;
										if (TotalSizeField.IsInteger())
										{
											TotalSize = TotalSizeField.AsUInt64();
										}
										else if (TotalSizeField.IsFloat())
										{
											TotalSize = static_cast<uint64>(TotalSizeField.AsDouble());
										}
										else if (TotalSizeField.IsString())
										{
											LexFromString(TotalSize, *FString(FUTF8ToTCHAR(TotalSizeField.AsString())));
										}
										else
										{
											return FText::GetEmpty();
										}
										return FText::FromString(SBuildArtifactContentsViewer::FormatSize(TotalSize));
									}
								}
								return FText::GetEmpty();
							})
							.Font_Lambda([this, ArtifactName, RegularFont, HighlightFont]
							{
								if (FArtifact* Artifact = NamedArtifacts.Find(ArtifactName))
								{
									if (Artifact->BuildRecord.BuildId == HighlightedBuildId)
									{
										return HighlightFont;
									}
								}
								return RegularFont;
							})
							.ColorAndOpacity_Lambda([this, ArtifactName]
							{
								if (FArtifact* Artifact = NamedArtifacts.Find(ArtifactName))
								{
									if (Artifact->BuildRecord.BuildId == HighlightedBuildId)
									{
										return FSlateColor(FStyleColors::AccentYellow);
									}
								}
								return FSlateColor::UseForeground();
							})
						]
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 8, 0)
				[
					SNew(SComboButton)
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("Artifact.SmallContext"))
					.ForegroundColor(FSlateColor::UseStyle())
					.OnGetMenuContent_Lambda([this, ArtifactName, IsBuildGroupArtifactEnabled]
						{
							return CreateArtifactMenuWidget(NamedArtifacts.Find(ArtifactName), ArtifactName);
						})
					.HasDownArrow(true)
				]
			];

		if (++Row >= MaxRows)
		{
			Column++;
			Row = 0;
		}
	}
}

void SBuildArtifactSelection::SetSelectedConfiguration(const TSharedPtr<FString> InSelectedConfiguration)
{
	using namespace UE::BuildSelection::Internal;

	SelectedConfiguration = InSelectedConfiguration;
	WriteSetting(TEXT("Configuration"), *SelectedConfiguration);
}

TSharedRef<SWidget> SBuildArtifactSelection::CreateArtifactMenuWidget(UE::BuildSelection::Internal::FArtifact* Artifact, const FString& ArtifactName)
{
	const bool bCloseAfterSelection = true;
	const bool bCloseSelfOnly = false;
	const bool bSearchable = false;
	const bool bRecursivelySearchable = false;

	FMenuBuilder MenuBuilder(bCloseAfterSelection,
		nullptr,
		TSharedPtr<FExtender>(),
		bCloseSelfOnly,
		&FCoreStyle::Get(),
		bSearchable,
		NAME_None,
		bRecursivelySearchable);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_ArtifactBrowseUrl", "Browse Build URL"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Artifact]
				{
					if (FCbFieldView BuildUrlField = Artifact->BuildRecord.Metadata["buildurl"]; BuildUrlField.HasValue() && !BuildUrlField.HasError())
					{
						FString Url(FUTF8ToTCHAR(BuildUrlField.AsString()));
						FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_CopyDownloadUrl", "Copy Download URL"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Artifact]
				{
					if (Artifact)
					{
						TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get();
						if (ServiceInstance)
						{
							// TODO: This URL format is specific to UE Build Storage - we should genericize this
							FString Url = FString::Printf(TEXT("%s/api/v2/builds/%s/%s/%s"),
								*FString(ServiceInstance->GetEffectiveDomain()),
								*Namespace,
								*Artifact->BuildRecord.BucketId,
								*FString(WriteToString<64>(Artifact->BuildRecord.BuildId)));
							FPlatformApplicationMisc::ClipboardCopy(*Url);
						}
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_ViewMetadata", "View Metadata"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Artifact, ArtifactName]
				{
					if (Artifact)
					{
						static const FName MetadataTabId(TEXT("BuildArtifactMetadata"));
						static TArray<TWeakPtr<SDockTab>> MetadataTabs;
						static TSharedPtr<SDockTab> PendingTab;

						// Register a spawner once; it returns the pre-built tab content
						if (!FGlobalTabmanager::Get()->HasTabSpawner(MetadataTabId))
						{
							FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MetadataTabId,
								FOnSpawnTab::CreateLambda([](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
								{
									return PendingTab.ToSharedRef();
								}))
								.SetMenuType(ETabSpawnerMenuType::Hidden);
						}

						// Create full display name including group
						FString FullDisplayName = ArtifactName;
						if (!GroupDisplayName.IsEmpty())
						{
							FullDisplayName = FString::Printf(TEXT("%s - %s"), *GroupDisplayName, *ArtifactName);
						}

						// Create the tab with real content
						TSharedRef<SDockTab> NewTab = SNew(SDockTab)
							.TabRole(ETabRole::DocumentTab)
							.Label(FText::FromString(FullDisplayName))
							[
								SNew(SBuildArtifactMetadataViewer)
								.Metadata(Artifact->BuildRecord.Metadata)
								.ArtifactName(FullDisplayName)
								.BuildId(Artifact->BuildRecord.BuildId)
							];

						// Find any existing live metadata tab
						TSharedPtr<SDockTab> ExistingTab;
						for (int32 i = MetadataTabs.Num() - 1; i >= 0; --i)
						{
							if (TSharedPtr<SDockTab> Tab = MetadataTabs[i].Pin())
							{
								ExistingTab = Tab;
							}
							else
							{
								MetadataTabs.RemoveAt(i);
							}
						}

						if (ExistingTab.IsValid())
						{
							// Dock alongside the existing metadata tab
							FGlobalTabmanager::Get()->InsertNewDocumentTab(
								MetadataTabId, FTabManager::FLiveTabSearch(), NewTab);
						}
						else
						{
							// No existing tab; open in a new window via the spawner
							PendingTab = NewTab;
							FGlobalTabmanager::Get()->TryInvokeTab(MetadataTabId);
							PendingTab.Reset();
						}

						MetadataTabs.Add(NewTab);
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	if (bCanViewContents)
	{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_ViewContents", "View Contents"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, Artifact, ArtifactName]
				{
					TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get();
					if (!ServiceInstance || !Artifact)
					{
						return;
					}

					FString FullDisplayName = ArtifactName;
					if (!GroupDisplayName.IsEmpty())
					{
						FullDisplayName = FString::Printf(TEXT("%s - %s"), *GroupDisplayName, *ArtifactName);
					}

					TSharedRef<SBuildArtifactContentsViewer> Viewer =
						SNew(SBuildArtifactContentsViewer)
						.ArtifactName(FullDisplayName)
						.OnStartFileDownload_Lambda(
							[ServiceInstance, Artifact, ArtifactName,
							 GroupDisplayName = this->GroupDisplayName, Namespace = this->Namespace]
							(FString&& DownloadSpec) -> UE::Zen::Build::FBuildServiceInstance::FBuildTransfer
						{
							FString DestFolder = FPaths::Combine(
								GetOpenFileTempRoot(), GroupDisplayName, ArtifactName);

							return ServiceInstance->StartBuildTransfer(
								Artifact->BuildRecord.BuildId,
								TEXT("Open"),
								DestFolder,
								Namespace,
								Artifact->BuildRecord.BucketId,
								FString(),
								FString(),
								MoveTemp(DownloadSpec),
								TArray<FString>(),
								TArray<FString>(),
								FStringView(),
								UE::Zen::Build::EBuildTransferRequestFlags::Append);
						});

					FString BuildIdString = FString(WriteToString<64>(Artifact->BuildRecord.BuildId));
					TWeakPtr<SBuildArtifactContentsViewer> WeakViewer = Viewer;
					ServiceInstance->ListBuildContents(Namespace, Artifact->BuildRecord.BucketId, BuildIdString,
						[WeakViewer](TMap<FString, UE::Zen::Build::FBuildServiceInstance::FBuildPart>&& BuildContents)
						{
							if (TSharedPtr<SBuildArtifactContentsViewer> Viewer = WeakViewer.Pin())
							{
								Viewer->QueueContents(MoveTemp(BuildContents));
							}
						});

					if (!FDialogCommands::IsRegistered())
					{
						FDialogCommands::Register();
					}

					TSharedPtr<SBuildArtifactContentsViewer> ViewerPtr = Viewer;
					TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
						.Title(FText::Format(LOCTEXT("ViewContentsTitle", "Build Contents: {0}"), FText::FromString(FullDisplayName)))
						.ClientSize(FVector2D(800, 600))
						.UseScrollBox(false)
						.HAlignContent(HAlign_Fill)
						.VAlignContent(VAlign_Fill)
						.Content()
						[
							Viewer
						]
						.Buttons({
							SCustomDialog::FButton(LOCTEXT("DownloadMarked", "Download Marked"))
								.SetIsEnabled(TAttribute<bool>::CreateLambda([ViewerPtr]()
								{
									return ViewerPtr.IsValid() && ViewerPtr->HasMarkedFiles();
								})),
							SCustomDialog::FButton(LOCTEXT("Close", "Close"))
						});

					int32 ButtonIndex = Dialog->ShowModal();
					if (ButtonIndex == 0 && ViewerPtr->HasMarkedFiles())
					{
						FDownloadSpec Spec = ViewerPtr->ComposeDownloadSpec();
						OnDownloadWithSpec.ExecuteIfBound(ArtifactName, *Artifact, Spec.SpecJSON, Spec.FullyMarkedPartNames);
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
