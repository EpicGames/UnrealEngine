// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildLogin.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "ZenServiceInstanceManager.h"

#define LOCTEXT_NAMESPACE "StorageServerBuild"

void SBuildLogin::Construct(const FArguments& InArgs)
{
	using namespace UE::Zen::Build;
	ZenServiceInstance = InArgs._ZenServiceInstance;
	BuildServiceInstance = InArgs._BuildServiceInstance;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		ServiceInstance->Connect(false, [this, SharedThis=AsShared(), ServiceInstance]
			(FBuildServiceInstance::EConnectionState ConnectionState,
				FBuildServiceInstance::EConnectionFailureReason FailureReason)
			{
				if (ConnectionState == FBuildServiceInstance::EConnectionState::ConnectionSucceeded)
				{
					ServiceInstance->RefreshNamespacesAndBuckets();
				}
				else
				{
					LastConnectionFailureReason = FailureReason;
				}
			});
	}
}

UE::Zen::Build::FBuildServiceInstance::EConnectionState SBuildLogin::GetEffectiveConnectionState(TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance) const
{
	using namespace UE::Zen::Build;
	FBuildServiceInstance::EConnectionState ConnectionState = ServiceInstance->GetConnectionState();
	if (!bTriggeredInteractiveLogin &&
		(ConnectionState == FBuildServiceInstance::EConnectionState::ConnectionFailed) &&
		(LastConnectionFailureReason == FBuildServiceInstance::EConnectionFailureReason::FailedToAcquireAccessToken)
		)
	{
		// If we've only tried non-interactive login and it failed due to not being able to acquire a token, treat it as if we've
		// not started the connection for UI purposes.  Showing a failure in that scenario is confusing messaging.
		ConnectionState = FBuildServiceInstance::EConnectionState::NotStarted;
	}

	return ConnectionState;
}

TSharedRef<SWidget> SBuildLogin::GetGridPanel()
{
	using namespace UE::Zen::Build;
	TSharedRef<SGridPanel> Panel = SNew(SGridPanel);

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	Panel->AddSlot(0, 0)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("BuildConnect_StatusLabel", "Status:"))
	];

	Panel->AddSlot(1, 0)
	.Padding(FMargin(2.0f, RowMargin))
	[
		SNew(STextBlock)
		.Text_Lambda([this]
		{
			if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
			{
				FBuildServiceInstance::EConnectionState ConnectionState = GetEffectiveConnectionState(ServiceInstance);

				switch (ConnectionState)
				{
				case FBuildServiceInstance::EConnectionState::NotStarted:
					return LOCTEXT("BuildConnect_StatusValueNotConnected", "Not Connected");
				case FBuildServiceInstance::EConnectionState::ConnectionInProgress:
					return LOCTEXT("BuildConnect_StatusValueConnecting", "Connecting...");
				case FBuildServiceInstance::EConnectionState::ConnectionSucceeded:
					{
						return FText::Format(LOCTEXT("BuildConnect_StatusValueConnected", "Connected to {0}"),
							FText::FromStringView(ServiceInstance->GetEffectiveDomain()));
					}
				case FBuildServiceInstance::EConnectionState::ConnectionFailed:
					return LOCTEXT("BuildConnect_StatusValueConnectionFailed", "Connection failed");
				}
			}
			return LOCTEXT("BuildConnect_StatusValueError", "Error");
		})
		.Visibility_Lambda([this]
		{
			if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
			{
				if (ServiceInstance->GetConnectionState() == FBuildServiceInstance::EConnectionState::NotStarted)
				{
					return EVisibility::Collapsed;
				}
			}
			return EVisibility::Visible;
		})
	];

	Panel->AddSlot(2, 0)
	.Padding(FMargin(2.0f, RowMargin))
	[
		SNew(SHorizontalBox)
		.Visibility_Lambda([this]
		{
			if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
			{
				if (ServiceInstance->GetConnectionState() != FBuildServiceInstance::EConnectionState::ConnectionInProgress)
				{
					return EVisibility::Visible;
				}
			}
			return EVisibility::Hidden;
		})
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("(")))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SHyperlink)
			.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
			.Text_Lambda([this]
			{
				if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
				{
					FBuildServiceInstance::EConnectionState ConnectionState = GetEffectiveConnectionState(ServiceInstance);

					switch (ConnectionState)
					{
					case FBuildServiceInstance::EConnectionState::ConnectionSucceeded:
					case FBuildServiceInstance::EConnectionState::ConnectionFailed:
						return LOCTEXT("BuildLogin_ReconnectLink", "reconnect");
					}
				}
				return LOCTEXT("BuildLogin_ConnectLink", "Click to connect");
			})
			.OnNavigate_Lambda([this]
			{
				if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
				{
					bTriggeredInteractiveLogin = true;
					ServiceInstance->Connect(true, [this, SharedThis=AsShared(), ServiceInstance]
						(FBuildServiceInstance::EConnectionState ConnectionState,
							FBuildServiceInstance::EConnectionFailureReason FailureReason)
						{
							if (ConnectionState == FBuildServiceInstance::EConnectionState::ConnectionSucceeded)
							{
								ServiceInstance->RefreshNamespacesAndBuckets();
							}
							else
							{
								LastConnectionFailureReason = FailureReason;
							}
						});
				}
			})
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT(")")))
		]
	];


	return Panel;
}

#undef LOCTEXT_NAMESPACE
