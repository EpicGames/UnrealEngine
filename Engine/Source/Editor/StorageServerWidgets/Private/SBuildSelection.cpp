// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildSelection.h"

#include "DesktopPlatformModule.h"
#include "Dialog/DialogCommands.h"
#include "Dialog/SCustomDialog.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextChronoFormatter.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/App.h"
#include "Misc/ExpressionParser.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Modules/BuildVersion.h"
#include "SBuildArtifactSelection.h"
#include "SBuildTransferFilters.h"
#include "SMultiSelectComboBox.h"
#include "String/ParseTokens.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "ZenBuildUtils.h"
#include "ZenServiceInstanceManager.h"
#include <atomic>

#define LOCTEXT_NAMESPACE "StorageServerBuild"

namespace UE::BuildSelection::Internal
{
	namespace FBuildGroupIds
	{
		const FName ColName = TEXT("Name");
		const FName ColCommit = TEXT("Commit");
		const FName ColCategory = TEXT("Category");
		const FName ColCreated = TEXT("Created");
	}
}

TPair<FRegexPattern, SBuildSelection::FKnownBuildType> SBuildSelection::MakeKnownBuildTypePattern(const FStringView InPattern, EBuildType Type, FText&& InText, int InDefaultSelectionOrdering)
{
	FString Pattern(InPattern);
	return TPair<FRegexPattern, FKnownBuildType>(Pattern, FKnownBuildType{Type, InText, InDefaultSelectionOrdering});
}

void SBuildSelection::Construct(const FArguments& InArgs)
{
	using namespace UE::BuildSelection::Internal;

	KnownBuildTypePatterns = {
		// These should be made more constinent and perhaps specified in INI
		MakeKnownBuildTypePattern(TEXT(".*oplog-?(.*)"), EBuildType::Oplog, LOCTEXT("BuildSelection_BuildType_Oplog", "Cook Snapshot") ),
		MakeKnownBuildTypePattern(TEXT(".*packaged-?build-?(.*)"), EBuildType::PackagedBuild, LOCTEXT("BuildSelection_BuildType_PackagedBuild", "Packaged Build"), 1),
		MakeKnownBuildTypePattern(TEXT(".*staged-?(.*?)-?build.*"), EBuildType::StagedBuild, LOCTEXT("BuildSelection_BuildType_StagedBuild", "Staged Build"), 2),
		MakeKnownBuildTypePattern(TEXT(".*ugs-?pcb-?(.*)"), EBuildType::EditorPreCompiledBinary, LOCTEXT("BuildSelection_BuildType_PreCompiledBinary", "Editor Pre-compiled Binary")),
		MakeKnownBuildTypePattern(TEXT(".*installed-?build-?(.*)"), EBuildType::EditorPreCompiledBinary, LOCTEXT("BuildSelection_BuildType_PreCompiledBinary", "Editor Pre-compiled Binary"))
	};

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		DesktopPlatform->EnumerateEngineInstallations(EngineInstallations);
	}

	// If the tool was launched with an explicit workspace root directory, use it to set the engine installation directory
	// and ensure that it is considered as an installation directory even if is not one of the items found by EnumerateEngineInstallations.
	// This can happen when an engine installation hasn't had the editor run in it yet, and hasn't yet registered itself in a way that is
	// discoverable by EnumerateEngineInstallations.
	FString WorkspaceRoot;
	if (FParse::Value(FCommandLine::Get(), TEXT("-WorkspaceRoot"), WorkspaceRoot))
	{
		WorkspaceRoot.TrimStartAndEndInline();
		WorkspaceRoot.TrimQuotesInline();
		FPaths::NormalizeDirectoryName(WorkspaceRoot);
		bool bFoundExisting = false;
		for (const TPair<FString, FString>& EngineInstallation : EngineInstallations)
		{
			if (FPaths::IsSamePath(WorkspaceRoot, EngineInstallation.Value))
			{
				bFoundExisting = true;
				WriteSetting(TEXT("UserSelectedEngineInstallation"), EngineInstallation.Key);
				SetUserSelectedProjectDictionaryRoot(EngineInstallation.Value);
				break;
			}
		}
		if (!bFoundExisting)
		{
			EngineInstallations.Add(FString(), WorkspaceRoot);
			WriteSetting(TEXT("UserSelectedEngineInstallation"), FString());
			SetUserSelectedProjectDictionaryRoot(WorkspaceRoot);
		}
	}

	// If the tool was launched with an explicit project file, use it to set the engine installation directory and the project
	// regardless of past settings.  These values should then persist as new settings.
	FString ProjectFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("-ProjectFile"), ProjectFile))
	{
		ProjectFile.TrimStartAndEndInline();
		ProjectFile.TrimQuotesInline();
		if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
		{
			for (const TPair<FString, FString>& EngineInstallation : EngineInstallations)
			{
				if (FPaths::IsUnderDirectory(ProjectFile, EngineInstallation.Value))
				{
					WriteSetting(TEXT("UserSelectedEngineInstallation"), EngineInstallation.Key);
					SetUserSelectedProjectDictionaryRoot(EngineInstallation.Value);
					break;
				}
			}
		}
		SetSelectedProject(MakeShared<FString>(FPaths::GetBaseFilename(ProjectFile).ToLower()));
	}

	FString UserSelectedEngineInstallation;
	if (ReadSetting(TEXT("UserSelectedEngineInstallation"), UserSelectedEngineInstallation))
	{
		if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
		{
			FString UserSelectedRootDir;
			if (DesktopPlatform->GetEngineRootDirFromIdentifier(UserSelectedEngineInstallation, UserSelectedRootDir))
			{
				SetUserSelectedProjectDictionaryRoot(UserSelectedRootDir);
			}
		}
	}

	for (bool& bAppendBuildNameToDestination : bAppendBuildNameToDestinations)
	{
		bAppendBuildNameToDestination = false;
	}

	for (int32 BuildTypeIndex = 0; BuildTypeIndex < (int32)EBuildType::Count; ++BuildTypeIndex)
	{
		ReadSetting(LexToString((EBuildType)BuildTypeIndex), TEXT("UserSelectedDestination"), UserSelectedDestinations[BuildTypeIndex]);
		FString AppendBuildNameToDestinationString;
		ReadSetting(LexToString((EBuildType)BuildTypeIndex), TEXT("UserSelectedAppendBuildNameToDestination"), AppendBuildNameToDestinationString);
		bAppendBuildNameToDestinations[BuildTypeIndex] = FCString::ToBool(*AppendBuildNameToDestinationString);
		ReadSetting(LexToString((EBuildType)BuildTypeIndex), TEXT("UserSelectedIncludeFilter"), UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter);
		ReadSetting(LexToString((EBuildType)BuildTypeIndex), TEXT("UserSelectedExcludeFilter"), UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter);
	}

	if (const TPair<FString, FString>* EngineInstallation = EngineInstallations.FindArbitraryElement())
	{
		SetFallbackProjectDictionaryRoot(EngineInstallation->Value);
	}

	ZenServiceInstance = InArgs._ZenServiceInstance;
	BuildServiceInstance = InArgs._BuildServiceInstance;
	PreBuildTransfer = InArgs._PreBuildTransfer;
	PreOplogBuildTransfer = InArgs._PreOplogBuildTransfer;
	OnBuildTransferStarted = InArgs._OnBuildTransferStarted;

	BuildGroupSortByColumn = FBuildGroupIds::ColCreated;
	BuildGroupSortMode = EColumnSortMode::Descending;

	FBuildVersion CurrentBuildVersion;
	if (FBuildVersion::TryRead(FBuildVersion::GetDefaultFileName(), CurrentBuildVersion))
	{
		BranchName = SanitizeBranch(CurrentBuildVersion.BranchName);

		EffectiveCompatibleChangelist = CurrentBuildVersion.GetEffectiveCompatibleChangelist();
	}

	// If the tool was launched with an explicit branch, use it to set the branch name
	// regardless of past settings.
	if (FParse::Value(FCommandLine::Get(), TEXT("-Branch"), BranchName))
	{
		BranchName = SanitizeBranch(BranchName);
	}

	if (BranchName.IsEmpty())
	{
		ReadSetting(TEXT("Branch"), BranchName);
	}
	
	FString IncludePreflightsString;
	if (ReadSetting(TEXT("IncludePreflights"), IncludePreflightsString))
	{
		bIncludePreflights = FCString::ToBool(*IncludePreflightsString);
	}

	if (TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		ServiceInstance->OnRefreshNamespacesAndBucketsComplete().AddSPLambda(this, [this]()
			{
				ExecuteOnGameThread(UE_SOURCE_LOCATION,
					[this]
					{
						RebuildLists();
					});
			});
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0, 0, 0, 0)
		[
			GetGridPanel()
		]
	];
}

void SBuildSelection::ActOnBuildArtifact(EBuildArtifactAction Action, const UE::Zen::Build::FBuildReference& BuildReference, bool bAllowRebuildLists)
{
	if (Action == EBuildArtifactAction::None)
	{
		return;
	}

	TArray<FStringView> BucketParts;
	UE::String::ParseTokens(BuildReference.Bucket,
		TCHAR('.'),
		[&BucketParts](FStringView BucketPart)
		{
			BucketParts.Add(BucketPart);
		});

	if (BucketParts.Num() < 3)
	{
		return;
	}

	bool bNeedListRebuild = false;
	if (!SelectedStream || *SelectedStream != BucketParts[2])
	{
		SetSelectedStream(MakeShared<FString>(BucketParts[2]));
		bNeedListRebuild = true;
	}
	if (!SelectedProject || *SelectedProject != BucketParts[0])
	{
		SetSelectedProject(MakeShared<FString>(BucketParts[0]));
		bNeedListRebuild = true;
	}
	if (!SelectedBuildType || *SelectedBuildType != BucketParts[1])
	{
		SetSelectedBuildType(MakeShared<FString>(BucketParts[1]));
		bNeedListRebuild = true;
	}

	{
		FWriteScopeLock WriteLock(TargetedBuildIdLock);
		TargetedBuildIdAction = Action;
		TargetedBuildId = BuildReference.BuildId;
	}

	if (bAllowRebuildLists && bNeedListRebuild)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[this]
			{
				RebuildLists();
			});
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[this]
			{
				EnforceTargetedBuildActionOnGroup();
				RefreshPresentedBuildGroups();
			});
	}
}

void SBuildSelection::EnforceTargetedBuildActionOnGroup()
{
	using namespace UE::BuildSelection::Internal;

	EBuildArtifactAction Action;
	FCbObjectId BuildId;
	{
		FReadScopeLock ReadLock(TargetedBuildIdLock);
		Action = TargetedBuildIdAction;
		BuildId = TargetedBuildId;
	}

	if (Action == EBuildArtifactAction::None)
	{
		return;
	}

	for (FBuildSelectionBuildGroupPtr BuildGroup : BuildGroups)
	{
		for (const TPair<FString, FArtifact>& NamedArtifact : BuildGroup->NamedArtifacts)
		{
			if (NamedArtifact.Value.BuildRecord.BuildId == BuildId)
			{
				BuildGroupListView->ClearSelection();
				BuildGroupListView->SetItemSelection(BuildGroup, true);
				BuildGroupListView->RequestScrollIntoView(BuildGroup);

				if (Action == EBuildArtifactAction::HighlightAndDownload)
				{
					DownloadItem(*BuildGroup, NamedArtifact.Key, NamedArtifact.Value);
				}
				return;
			}
		}
	}
}

SBuildSelection::EBuildType SBuildSelection::GetSelectedBuildType() const
{
	if (!SelectedBuildType)
	{
		return EBuildType::Unknown;
	}
	for (const TPair<FRegexPattern, FKnownBuildType>& KnownBuildTypeItem : KnownBuildTypePatterns)
	{
		FRegexMatcher Matcher(KnownBuildTypeItem.Key, *SelectedBuildType);
		if (Matcher.FindNext())
		{
			return KnownBuildTypeItem.Value.Type;
		}
	}
	return EBuildType::Unknown;
}

void SBuildSelection::RebuildLists()
{
	using namespace UE::Zen::Build;
	using namespace UE::BuildSelection::Internal;

	BuildListRefreshesInProgress.fetch_add(1);
	StreamList.Empty();
	ProjectList.Empty();
	BuildTypeList.Empty();
	PlatformList.Empty();

	TArray<FString> Namespaces;
	TMultiMap<FString, FString> NamespacesAndBuckets;

	if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		TArray<FString> Projects;
		TArray<FString> Streams;
		TArray<FString> BuildTypes;
		TArray<FString> Platforms;
		NamespacesAndBuckets = ServiceInstance->GetNamespacesAndBuckets();

		auto StringToSegmentViews = [](const FString& Str, TArray<FStringView>& OutViews)
			{
				FStringView WorkingStringView(Str);
				int32 CurrentIndex = 0;
				while (WorkingStringView.FindChar(TCHAR('.'), CurrentIndex))
				{
					if (CurrentIndex != 0)
					{
						OutViews.Add(WorkingStringView.Left(CurrentIndex));
					}
					WorkingStringView.RightChopInline(CurrentIndex + 1);
				}
				if (!WorkingStringView.IsEmpty())
				{
					OutViews.Add(WorkingStringView);
				}
			};

		NamespacesAndBuckets.GetKeys(Namespaces);

		auto ConvertToSharedPtrs = [](TArray<FString>& Strings, TArray<TSharedPtr<FString>>& SharedStrings)
			{
				Strings.StableSort();
				for (FString& String : Strings)
				{
					SharedStrings.Add(MakeShared<FString>(MoveTemp(String)));
				}
			};

		const uint32 SegmentIndexProject = 0;
		const uint32 SegmentIndexBuildType = 1;
		const uint32 SegmentIndexStream = 2;
		const uint32 SegmentIndexPlatform = 3;
		const uint32 SegmentIndexNum = 4;

		// Stream list generation and selection conforming
		int32 DefaultStreamIndex = 0;
		TMultiMap<FStringView, TArray<FStringView>> NamespacesToBucketSegmentViews;
		for (const TPair<FString, FString>& NamespaceAndBucket : NamespacesAndBuckets)
		{
			TArray<FStringView>& BucketSegmentViews = NamespacesToBucketSegmentViews.Add(NamespaceAndBucket.Key);
			StringToSegmentViews(NamespaceAndBucket.Value, BucketSegmentViews);

			if (BucketSegmentViews.Num() == SegmentIndexNum)
			{
				int32 CurrentIndex = Streams.AddUnique(FString(BucketSegmentViews[SegmentIndexStream]));
				if (!BranchName.IsEmpty() && (Streams[CurrentIndex] == BranchName))
				{
					DefaultStreamIndex = CurrentIndex;
				}
			}
		}
		FString DefaultStreamValue = Streams.Num() > 0 ? Streams[DefaultStreamIndex] : TEXT("");
		ConvertToSharedPtrs(Streams, StreamList);
		// Must search to find DefaultStreamIndex because it will have re-sorted in ConverToSharedPtrs
		const TSharedPtr<FString>* DefaultStreamListItem = StreamList.FindByPredicate([&DefaultStreamValue](const TSharedPtr<FString>& Item)
			{
				return *Item == DefaultStreamValue;
			});
		if (!SelectedStream && DefaultStreamListItem)
		{
			SetSelectedStream(*DefaultStreamListItem);
		}
		ConformSelection(SelectedStream, StreamList);

		// Project list generation and selection conforming
		for (const TPair<FStringView, TArray<FStringView>>& NamespaceToBucketSegmentViews : NamespacesToBucketSegmentViews)
		{
			const TArray<FStringView>& BucketSegmentViews = NamespaceToBucketSegmentViews.Value;
			if (BucketSegmentViews.Num() == SegmentIndexNum)
			{
				if (BucketSegmentViews[SegmentIndexStream] != (SelectedStream ? *SelectedStream : Streams[0]))
				{
					continue;
				}

				Projects.AddUnique(FString(BucketSegmentViews[SegmentIndexProject]));
			}
		}

		if (!SelectedProject)
		{
			if (FString PastProject; ReadSetting(TEXT("Project"), PastProject))
			{
				SelectedProject = MakeShared<FString>(MoveTemp(PastProject));
			}
		}
		ConvertToSharedPtrs(Projects, ProjectList);
		ConformSelection(SelectedProject, ProjectList);

		// BuildType list generation and selection conforming
		for (const TPair<FStringView, TArray<FStringView>>& NamespaceToBucketSegmentViews : NamespacesToBucketSegmentViews)
		{
			const TArray<FStringView>& BucketSegmentViews = NamespaceToBucketSegmentViews.Value;
			if (BucketSegmentViews.Num() == SegmentIndexNum)
			{
				if (BucketSegmentViews[SegmentIndexStream] != (SelectedStream ? *SelectedStream : Streams[0]))
				{
					continue;
				}

				if (BucketSegmentViews[SegmentIndexProject] != (SelectedProject ? *SelectedProject : Projects[0]))
				{
					continue;
				}

				BuildTypes.AddUnique(FString(BucketSegmentViews[SegmentIndexBuildType]));
			}
		}

		if (!SelectedBuildType)
		{
			if (FString PastBuildType; ReadSetting(TEXT("BuildType"), PastBuildType))
			{
				SelectedBuildType = MakeShared<FString>(MoveTemp(PastBuildType));
			}
		}
		ConvertToSharedPtrs(BuildTypes, BuildTypeList);
		TArray<TSharedPtr<FString>> BuildTypeListSortedByDefaultPreference = BuildTypeList;
		BuildTypeListSortedByDefaultPreference.StableSort([this] (const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
		{
			int ASelectionOrdering = ConvertBuildTypeToDefaultSelectionOrdering(*A);
			int BSelectionOrdering = ConvertBuildTypeToDefaultSelectionOrdering(*B);

			if (ASelectionOrdering != BSelectionOrdering)
			{
				return ASelectionOrdering >= BSelectionOrdering;
			}

			return *A < *B;
		});
		ConformSelection(SelectedBuildType, BuildTypeListSortedByDefaultPreference);

		// Platform list generation
		for (const TPair<FStringView, TArray<FStringView>>& NamespaceToBucketSegmentViews : NamespacesToBucketSegmentViews)
		{
			const TArray<FStringView>& BucketSegmentViews = NamespaceToBucketSegmentViews.Value;
			if (BucketSegmentViews.Num() == SegmentIndexNum)
			{
				if (BucketSegmentViews[SegmentIndexStream] != (SelectedStream ? *SelectedStream : Streams[0]))
				{
					continue;
				}

				if (BucketSegmentViews[SegmentIndexProject] != (SelectedProject ? *SelectedProject : Projects[0]))
				{
					continue;
				}

				if (BucketSegmentViews[SegmentIndexBuildType] != (SelectedBuildType ? *SelectedBuildType : BuildTypes[0]))
				{
					continue;
				}

				Platforms.AddUnique(FString(BucketSegmentViews[SegmentIndexPlatform]));
			}
		}
		ConvertToSharedPtrs(Platforms, PlatformList);
	}

	ExecuteOnGameThread(UE_SOURCE_LOCATION,
		[this]
		{
			StreamWidget->RefreshOptions();
			StreamWidget->SetSelectedItem(SelectedStream);
			ProjectWidget->RefreshOptions();
			ProjectWidget->SetSelectedItem(SelectedProject);
			BuildTypeWidget->RefreshOptions();
			BuildTypeWidget->SetSelectedItem(SelectedBuildType);
			RegenerateActivePlatformFilters();
		});

	if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		BuildGroups.Empty();
		PresentedBuildGroups.Reset();

		if (NamespacesAndBuckets.IsEmpty() || !SelectedProject || !SelectedBuildType || !SelectedStream)
		{
			BuildListRefreshesInProgress.fetch_sub(1);
			return;
		}

		FString BucketRegex = FString::Printf(TEXT("%s\\.%s\\.%s\\..*"), **SelectedProject, **SelectedBuildType, **SelectedStream);
		FString BucketPrefix = FString::Printf(TEXT("%s.%s.%s."), **SelectedProject, **SelectedBuildType, **SelectedStream);

		TArray<FString> NamespacesToQuery;

		for (const FString& Namespace : Namespaces)
		{
			TArray<FString> BucketsInNamespace;
			NamespacesAndBuckets.MultiFind(Namespace, BucketsInNamespace);
			for (const FString& Bucket : BucketsInNamespace)
			{
				if (Bucket.StartsWith(BucketPrefix))
				{
					NamespacesToQuery.Add(Namespace);
					break;
				}
			}
		}

		TSharedPtr<FListBuildsState> PendingQueryState = MakeShared<FListBuildsState>();
		PendingQueryState->PendingQueries = NamespacesToQuery.Num();
		PendingQueryState->QueryState.SetNum(NamespacesToQuery.Num());
		uint32 QueryIndex = 0;

		++BuildRefreshGeneration;

		for (const FString& Namespace : NamespacesToQuery)
		{
			ServiceInstance->ListBuildsAcrossBuckets(Namespace, BucketRegex, FCbObject(),
			[this, QueryIndex, Namespace = FString(Namespace),
				ExpectedBuildRefreshGeneration = BuildRefreshGeneration.load(), PendingQueryState]
			(TArray<FBuildServiceInstance::FBuildRecord>&& Results) mutable
				{
					FBuildState& NewBuildState = PendingQueryState->QueryState[QueryIndex];
					NewBuildState.Namespace = MoveTemp(Namespace);
					NewBuildState.Results = MoveTemp(Results);

					if (--PendingQueryState->PendingQueries == 0)
					{
						// All queries complete
						if (ExpectedBuildRefreshGeneration == BuildRefreshGeneration)
						{
							// Expected generation is the current generation
							RegenerateBuildGroups(*PendingQueryState);

							ExecuteOnGameThread(UE_SOURCE_LOCATION,
								[this]
								{
									EnforceTargetedBuildActionOnGroup();
									PostViewFilterChange();
									BuildListRefreshesInProgress.fetch_sub(1);
								});
						}
						else
						{
							// Expected generation is not the current generation
							ExecuteOnGameThread(UE_SOURCE_LOCATION,
								[this]
								{
									BuildListRefreshesInProgress.fetch_sub(1);
								});
						}
					}
				});
			++QueryIndex;
		}

		if (NamespacesToQuery.IsEmpty())
		{
			BuildListRefreshesInProgress.fetch_sub(1);
		}
	}
	else
	{
		BuildListRefreshesInProgress.fetch_sub(1);
	}
}

void SBuildSelection::RegenerateBuildGroups(UE::Zen::Build::FListBuildsState& ListBuildsState)
{
	using namespace UE::Zen::Build;
	using namespace UE::BuildSelection::Internal;

	BuildGroups.Empty();
	PresentedBuildGroups.Reset();
	TMultiMap<FString, FString> ArtifactNameMap;
	TSet<FString> ArtifactConfigurationSet;
	bool bHasNoneArtifactConfiguration = false;

	auto MakeGroupKey = [](const FString& Namespace, const FString& CommitIdentifier, const FBuildServiceInstance::FBuildRecord& BuildRecord)
	{
		if (FCbFieldView BuildGroupField = BuildRecord.Metadata.FindIgnoreCase("buildgroup"); BuildGroupField.HasValue() && !BuildGroupField.HasError())
		{
			return FString(WriteToString<64>(Namespace, ".", BuildGroupField.AsString()));
		}
		if (FCbFieldView NameField = BuildRecord.Metadata.FindIgnoreCase("name"); NameField.HasValue() && !NameField.HasError())
		{
			FString CandidateKeyName = FString(WriteToString<64>(Namespace, ".", NameField.AsString()));
			// TODO: This name manipulation needs to be removed when the metadata is more consistent.
			int32 CLIndex = CandidateKeyName.Find(TEXT("-CL-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (CLIndex != INDEX_NONE)
			{
				int32 DashIndex = CandidateKeyName.Find(TEXT("-"), ESearchCase::IgnoreCase, ESearchDir::FromStart, CLIndex + 4);
				if (DashIndex != INDEX_NONE)
				{
					CandidateKeyName.LeftInline(DashIndex);
				}
				else
				{
					int32 DotIndex = CandidateKeyName.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, CLIndex + 4);
					if (DotIndex != INDEX_NONE)
					{
						CandidateKeyName.LeftInline(DotIndex);
					}
				}
			}
			return CandidateKeyName;
		}
		if (!CommitIdentifier.IsEmpty())
		{
			return FString(WriteToString<64>(Namespace, ".", CommitIdentifier));
		}
		return FString(WriteToString<64>(BuildRecord.BuildId));
	};

	TMap<FString, TSharedPtr<FBuildGroup>> KeyedGroups;
	for (FBuildState& BuildState : ListBuildsState.QueryState)
	{
		for (FBuildServiceInstance::FBuildRecord& BuildRecord : BuildState.Results)
		{
			FString CommitIdentifier = BuildRecord.GetCommitIdentifier();

			FString GroupKey = MakeGroupKey(BuildState.Namespace, CommitIdentifier, BuildRecord);
			TSharedPtr<FBuildGroup>& BuildGroup = KeyedGroups.FindOrAdd(GroupKey);
			if (!BuildGroup)
			{
				BuildGroup = MakeShared<FBuildGroup>();
			}

			FString ArtifactName;
			if (FCbFieldView NameView = BuildRecord.Metadata.FindIgnoreCase("name"); NameView.HasValue() && !NameView.HasError())
			{
				ArtifactName = FUTF8ToTCHAR(NameView.AsString());

				// TODO: This name manipulation needs to be removed when the metadata is more consistent.
				int32 TruncationIndex;
				int32 CLIndex = ArtifactName.Find(TEXT("-CL-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (CLIndex != INDEX_NONE)
				{
					ArtifactName.RightChopInline(CLIndex + 4);
					if (ArtifactName.FindChar(TCHAR('-'), TruncationIndex))
					{
						ArtifactName.RightChopInline(TruncationIndex + 1);
					}
				}
				if (ArtifactName.FindLastChar(TCHAR('.'), TruncationIndex))
				{
					ArtifactName.RightChopInline(TruncationIndex + 1);
				}
			}

			FString Configuration;
			if (FCbFieldView ConfigurationField = BuildRecord.Metadata.FindIgnoreCase("Configuration"); ConfigurationField.HasValue() && !ConfigurationField.HasError())
			{
				Configuration = FUTF8ToTCHAR(ConfigurationField.AsString());
			}
			
			if (BuildGroup->DisplayName.IsEmpty())
			{
				FString Job;
				if (FCbFieldView JobField = BuildRecord.Metadata.FindIgnoreCase("job"); JobField.HasValue() && !JobField.HasError())
				{
					Job = FUTF8ToTCHAR(JobField.AsString());
				}

				FString Category;
				if (FCbFieldView TemplateIdField = BuildRecord.Metadata.FindIgnoreCase("hordeTemplateId"); TemplateIdField.HasValue() && !TemplateIdField.HasError())
				{
					Category = FUTF8ToTCHAR(TemplateIdField.AsString());
				}

				bool bIsPreflight = false;
				if (FCbFieldView IsPreflightField = BuildRecord.Metadata.FindIgnoreCase("ispreflight"); IsPreflightField.HasValue() && !IsPreflightField.HasError())
				{
					FString IsPreflightString;
					IsPreflightString = FUTF8ToTCHAR(IsPreflightField.AsString());
					bIsPreflight = IsPreflightString.ToBool();
				}

				FDateTime CreatedAt;
				if (FCbFieldView CreatedAtField = BuildRecord.Metadata.FindIgnoreCase("createdAt"); CreatedAtField.HasValue() && !CreatedAtField.HasError())
				{
					if (CreatedAtField.IsString())
					{
						FDateTime::ParseIso8601(FUTF8ToTCHAR(CreatedAtField.AsString()).Get(), CreatedAt);
					}
					else if (CreatedAtField.IsDateTime())
					{
						CreatedAt = CreatedAtField.AsDateTime();
					}
				}

				FString ItemName;
				FCbFieldView GroupNameView = BuildRecord.Metadata.FindIgnoreCase("buildgroup");
				if (!GroupNameView.HasValue())
				{
					GroupNameView = BuildRecord.Metadata.FindIgnoreCase("name");
				}
				if (FCbFieldView NameView = GroupNameView; NameView.HasValue() && !NameView.HasError())
				{
					// TODO: This name manipulation needs to be removed when the metadata is more consistent.
					ItemName = FUTF8ToTCHAR(NameView.AsString());
				}
				else
				{
					ItemName = *WriteToString<64>(BuildRecord.BuildId);
				}

				BuildGroup->Namespace = BuildState.Namespace;
				BuildGroup->DisplayName = ItemName;
				BuildGroup->CommitIdentifier = CommitIdentifier;
				BuildGroup->Category = Category;
				BuildGroup->CreatedAt = CreatedAt;
				BuildGroup->Job = Job;
				BuildGroup->bIsPreflight = bIsPreflight;
			}

			TSet<FName> AllConfigurationNames;
			if (Configuration.IsEmpty())
			{
				bHasNoneArtifactConfiguration = true;
				ArtifactNameMap.EmplaceUnique(TEXT("None"), ArtifactName);
			}
			else
			{
				TArray<FString> AllConfigurations;
				Configuration.ParseIntoArrayWS(AllConfigurations, TEXT("+"));
				for (const FString& IndividualConfiguration : AllConfigurations)
				{
					AllConfigurationNames.Add(FName(IndividualConfiguration));
					ArtifactConfigurationSet.Add(IndividualConfiguration);
					ArtifactNameMap.EmplaceUnique(IndividualConfiguration, ArtifactName);
				}
			}
			BuildGroup->NamedArtifacts.FindOrAdd(ArtifactName, { MoveTemp(AllConfigurationNames), MoveTemp(BuildRecord) });
		}
	}

	BuildArtifactSelection->SetAvailableConfigurations(MoveTemp(ArtifactConfigurationSet), MoveTemp(ArtifactNameMap));
	KeyedGroups.GenerateValueArray(BuildGroups);
	RefreshPresentedBuildGroups();
}

void SBuildSelection::RefreshPresentedBuildGroups()
{
	using namespace UE::BuildSelection::Internal;

	PresentedBuildGroups.Reset();
	for (TSharedPtr<FBuildGroup> BuildGroup : BuildGroups)
	{
		if (DoesPassAllViewFilters(BuildGroup))
		{
			PresentedBuildGroups.Add(BuildGroup);
		}
	}

	if (BuildGroupSortMode == EColumnSortMode::None)
	{
		return;
	}

	// Sorting
	if (BuildGroupSortMode == EColumnSortMode::Ascending)
	{
		if (BuildGroupSortByColumn == FBuildGroupIds::ColName)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (A->DisplayName < B->DisplayName);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCommit)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (A->CommitIdentifier < B->CommitIdentifier);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCategory)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (A->Category < B->Category);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCreated)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (A->CreatedAt < B->CreatedAt);
			});
		}
	}
	else
	{
		if (BuildGroupSortByColumn == FBuildGroupIds::ColName)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (B->DisplayName < A->DisplayName);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCommit)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (B->CommitIdentifier < A->CommitIdentifier);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCategory)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (B->Category < A->Category);
			});
		}
		else if (BuildGroupSortByColumn == FBuildGroupIds::ColCreated)
		{
			PresentedBuildGroups.Sort([this](const TSharedPtr<FBuildGroup>& A, const TSharedPtr<FBuildGroup>& B) {
				return (B->CreatedAt < A->CreatedAt);
			});
		}
	}
}

void SBuildSelection::RegenerateActivePlatformFilters()
{
	ActivePlatformFilters.Empty();
	for (TSharedPtr<FString> Platform : PlatformList)
	{
		if (Platform && PlatformFiltersWidget)
		{
			if (PlatformFiltersWidget->IsChecked(*Platform))
			{
				ActivePlatformFilters.Add(*Platform);
			}
		}
	}
}

void SBuildSelection::ValidateBuildGroupSelection()
{
	using namespace UE::BuildSelection::Internal;

	BuildGroupListView->UpdateSelectionSet();
	TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	EBuildArtifactAction Action;
	FCbObjectId BuildId;
	{
		FReadScopeLock ReadLock(TargetedBuildIdLock);
		Action = TargetedBuildIdAction;
		BuildId = TargetedBuildId;
	}

	for (const FBuildSelectionBuildGroupPtr& SelectedItem : SelectedItems)
	{
		if (Action != EBuildArtifactAction::None)
		{
			// If a row contains a targeted build that we're actively trying to highlight, we won't force it to be de-selected due to filter criteria
			for (const TPair<FString, FArtifact>& NamedArtifact : SelectedItem->NamedArtifacts)
			{
				if (NamedArtifact.Value.BuildRecord.BuildId == BuildId)
				{
					continue;
				}
			}
		}
		if (!BuildGroupIsSelectableOrNavigable(SelectedItem))
		{
			BuildGroupListView->SetItemSelection(SelectedItem, false);
		}
	}
}

TSharedRef<SWidget> SBuildSelection::OnGenerateTextBlockFromString(TSharedPtr<FString> Item)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*Item));
}

TSharedRef<SWidget> SBuildSelection::OnGenerateBuildTypeTextBlockFromString(TSharedPtr<FString> Item)
{
	return SNew(STextBlock)
		.Text(Item ? ConvertBuildTypeToText(*Item) : FText::GetEmpty());
}

bool SBuildSelection::DoesPassAllViewFilters(TSharedPtr<FBuildGroup> InItem) const
{
	using namespace UE::BuildSelection::Internal;

	if (!InItem)
	{
		return false;
	}

	EBuildArtifactAction Action;
	FCbObjectId BuildId;
	{
		FReadScopeLock ReadLock(TargetedBuildIdLock);
		Action = TargetedBuildIdAction;
		BuildId = TargetedBuildId;
	}

	// If a row contains a targeted build and is selected, it remains visible regardless of view filters
	for (const TPair<FString, FArtifact>& NamedArtifact : InItem->NamedArtifacts)
	{
		if (NamedArtifact.Value.BuildRecord.BuildId == BuildId)
		{
			TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
			if (!SelectedItems.IsEmpty())
			{
				for (const FBuildSelectionBuildGroupPtr& SelectedItem : SelectedItems)
				{
					if (SelectedItem == InItem)
					{
						return true;
					}
				}
			}
		}
	}

	for (const FString& ActivePlatformFilter : ActivePlatformFilters)
	{
		bool bHasPlatform = false;
		for (const TPair<FString, FArtifact>& NamedArtifact : InItem->NamedArtifacts)
		{
			if (NamedArtifact.Value.BuildRecord.BucketId.EndsWith(ActivePlatformFilter) &&
				NamedArtifact.Value.BuildRecord.BucketId.Len() > ActivePlatformFilter.Len() &&
				NamedArtifact.Value.BuildRecord.BucketId[NamedArtifact.Value.BuildRecord.BucketId.Len() - ActivePlatformFilter.Len() - 1] == TCHAR('.'))
			{
				bHasPlatform = true;
				break;
			}
		}
		if (!bHasPlatform)
		{
			return false;
		}
	}

	if (!bIncludePreflights && InItem->bIsPreflight)
	{
		return false;
	}

	if (SelectedTextFilter.IsEmpty())
	{
		return true;
	}

	if (InItem->Namespace.Contains(SelectedTextFilter) ||
		InItem->DisplayName.Contains(SelectedTextFilter) ||
		InItem->CommitIdentifier.Contains(SelectedTextFilter) ||
		InItem->Category.Contains(SelectedTextFilter) ||
		InItem->Job.Contains(SelectedTextFilter))
	{
		return true;
	}

	if (FText::AsDateTime(InItem->CreatedAt, EDateTimeStyle::Short, EDateTimeStyle::Short, FString(), FInternationalization::Get().GetCulture(FPlatformMisc::GetDefaultLocale())).ToString().Contains(SelectedTextFilter))
	{
		return true;
	}

	return false;
}

void SBuildSelection::PostViewFilterChange()
{
	RefreshPresentedBuildGroups();
	ValidateBuildGroupSelection();
	BuildGroupListView->Invalidate(EInvalidateWidgetReason::LayoutAndVolatility);
	BuildGroupListView->RequestListRefresh();
}

bool SBuildSelection::BuildGroupIsSelectableOrNavigable(FBuildSelectionBuildGroupPtr InItem) const
{
	return DoesPassAllViewFilters(InItem);
}

TSharedRef<ITableRow> SBuildSelection::GenerateBuildGroupRow(FBuildSelectionBuildGroupPtr InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	return SNew(SBuildGroupTableRow, InOwningTable, InItem);
}

TSharedPtr<SWidget> SBuildSelection::OnGetBuildGroupContextMenuContent() const
{
	TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();

	if (SelectedItems.IsEmpty())
	{
		return nullptr;
	}

	const bool bCloseAfterSelection = true;
	const bool bCloseSelfOnly = false;
	const bool bSearchable = false;
	const bool bRecursivelySearchable = false;
	FMenuBuilder MenuBuilder(bCloseAfterSelection,
		nullptr,
		TSharedPtr<FExtender>(),
		bCloseSelfOnly,
		&FCoreStyle::Get(),
		bSearchable,
		NAME_None,
		bRecursivelySearchable);


	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_CopyName", "Copy name"),
		LOCTEXT("BuildSelection_CopyName_ToolTip", "Copies the name of the build group"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
				{
					TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
					if (SelectedItems.Num() == 1)
					{
						FPlatformApplicationMisc::ClipboardCopy(*SelectedItems[0]->DisplayName);
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildSelection_CopyCommit", "Copy commit"),
		LOCTEXT("BuildSelection_CopyCommit_ToolTip", "Copies the commit or changelist number of the build group"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
				{
					TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
					if (SelectedItems.Num() == 1)
					{
						FPlatformApplicationMisc::ClipboardCopy(*SelectedItems[0]->CommitIdentifier);
					}
				})
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);


	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SBuildSelection::RefreshArtifactSelectionWidget(FBuildSelectionBuildGroupPtr Item)
{
	EBuildArtifactAction Action;
	{
		FReadScopeLock ReadLock(TargetedBuildIdLock);
		Action = TargetedBuildIdAction;
	}

	FCbObjectId BuildIdToHighlight;
	if (Item && (Action != EBuildArtifactAction::None))
	{
		FCbObjectId BuildId;
		{
			FWriteScopeLock WriteLock(TargetedBuildIdLock);
			Action = TargetedBuildIdAction;
			TargetedBuildIdAction = EBuildArtifactAction::None;
			if (Action != EBuildArtifactAction::None)
			{
				BuildIdToHighlight = TargetedBuildId;
			}
		}
	}
	BuildArtifactSelection->Refresh(Item ? FStringView(Item->Namespace) : FStringView(), Item ? Item->DisplayName : FStringView(), Item ? &Item->NamedArtifacts : nullptr, BuildIdToHighlight, GetSelectedBuildType() != EBuildType::Oplog);
}

void SBuildSelection::BuildGroupSelectionChanged(FBuildSelectionBuildGroupPtr Item, ESelectInfo::Type SelectInfo)
{
	RefreshPresentedBuildGroups();
	RefreshArtifactSelectionWidget(Item);
}

EColumnSortMode::Type SBuildSelection::GetColumnSortMode(const FName ColumnId) const
{
	if (BuildGroupSortByColumn == ColumnId)
	{
		return BuildGroupSortMode;
	}

	return EColumnSortMode::None;
}

void SBuildSelection::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	BuildGroupSortByColumn = ColumnId;
	BuildGroupSortMode = InSortMode;

	RefreshPresentedBuildGroups();
	ExecuteOnGameThread(UE_SOURCE_LOCATION,
		[this]
		{
			PostViewFilterChange();
		});
}

FText SBuildSelection::ConvertBuildTypeToText(const FString& InBuildType)
{
	for (const TPair<FRegexPattern, FKnownBuildType>& KnownBuildTypeItem : KnownBuildTypePatterns)
	{
		FRegexMatcher Matcher(KnownBuildTypeItem.Key, InBuildType);
		if (Matcher.FindNext())
		{
			FString OptionalCaptureGroupString = Matcher.GetCaptureGroup(1);
			if (OptionalCaptureGroupString.IsEmpty())
			{
				return KnownBuildTypeItem.Value.UserText;
			}
			return FText::Format(LOCTEXT("BuildSelection_KnownBuildTypeWithCaptureGroupFormat", "{0} ({1})"), KnownBuildTypeItem.Value.UserText, FText::FromString(OptionalCaptureGroupString));
		}
	}

	return FText::FromString(InBuildType);
}

int SBuildSelection::ConvertBuildTypeToDefaultSelectionOrdering(const FString& InBuildType)
{
	for (const TPair<FRegexPattern, FKnownBuildType>& KnownBuildTypeItem : KnownBuildTypePatterns)
	{
		FRegexMatcher Matcher(KnownBuildTypeItem.Key, InBuildType);
		if (Matcher.FindNext())
		{
			return KnownBuildTypeItem.Value.DefaultSelectionOrdering;
		}
	}

	return 0;
}

void SBuildSelection::SetUserSelectedDestination(const FStringView InDestination)
{
	using namespace UE::BuildSelection::Internal;
	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;
	UserSelectedDestinations[BuildTypeIndex] = InDestination;

	if (BuildType != EBuildType::Oplog)
	{
		for (const TPair<FString, FString>& EngineInstallation : EngineInstallations)
		{
			if (FPaths::IsUnderDirectory(UserSelectedDestinations[BuildTypeIndex], EngineInstallation.Value))
			{
				WriteSetting(TEXT("UserSelectedEngineInstallation"), EngineInstallation.Key);
				SetUserSelectedProjectDictionaryRoot(EngineInstallation.Value);
				break;
			}
		}
	}

	WriteSetting(LexToString(BuildType), TEXT("UserSelectedDestination"), InDestination);
}

void SBuildSelection::SetUserSelectedAppendBuildNameToDestination(bool bAppendBuildNameToDestination)
{
	using namespace UE::BuildSelection::Internal;
	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;
	bAppendBuildNameToDestinations[BuildTypeIndex] = bAppendBuildNameToDestination;

	WriteSetting(LexToString(BuildType), TEXT("UserSelectedAppendBuildNameToDestination"), ::LexToString(bAppendBuildNameToDestination));
}

void SBuildSelection::SetUserSelectedTransferFilter(const FStringView InIncludeFilter, const FStringView InExcludeFilter)
{
	using namespace UE::BuildSelection::Internal;
	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;
	UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter = InIncludeFilter;
	UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter = InExcludeFilter;

	WriteSetting(LexToString(BuildType), TEXT("UserSelectedIncludeFilter"), InIncludeFilter);
	WriteSetting(LexToString(BuildType), TEXT("UserSelectedExcludeFilter"), InExcludeFilter);
}

FString SBuildSelection::GetUserSelectedDestination() const
{
	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;
	return BuildType == EBuildType::Oplog ? SanitizeForZenId(UserSelectedDestinations[BuildTypeIndex]) : SanitizeForPath(UserSelectedDestinations[BuildTypeIndex]);
}

FString SBuildSelection::GetDefaultDestination() const
{
	EBuildType BuildType = GetSelectedBuildType();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString ProjectFilename;
	if (SelectedProject.IsValid())
	{
		ProjectFilename = FUProjectDictionary::GetDefault().GetProjectPathForGame(**SelectedProject);

		if (ProjectFilename.IsEmpty() && UserSelectedProjectDictionary)
		{
			ProjectFilename = UserSelectedProjectDictionary->GetProjectPathForGame(**SelectedProject);
		}

		if (ProjectFilename.IsEmpty() && FallbackProjectDictionary)
		{
			ProjectFilename = FallbackProjectDictionary->GetProjectPathForGame(**SelectedProject);
		}
	}

	if (ProjectFilename.IsEmpty())
	{
		if (BuildType == EBuildType::Oplog)
		{
			if (SelectedProject.IsValid())
			{
				return *SelectedProject;
			}
			else
			{
				return TEXT("DownloadedBuilds");
			}
		}

		FString DownloadBase;
		if (!UserSelectedProjectDictionaryRootDir.IsEmpty())
		{
			DownloadBase = UserSelectedProjectDictionaryRootDir;
		}
		else if (!FallbackProjectDictionaryRootDir.IsEmpty())
		{
			DownloadBase = FallbackProjectDictionaryRootDir;
		}
		else
		{
			DownloadBase = FPaths::EngineSavedDir();
		}

		return PlatformFile.ConvertToAbsolutePathForExternalAppForRead(
				*FPaths::Combine(DownloadBase, TEXT("DownloadedBuilds"))
			);
	}

	switch (BuildType)
	{
	case EBuildType::StagedBuild:
		return PlatformFile.ConvertToAbsolutePathForExternalAppForRead(
				*FPaths::Combine(FPaths::GetPath(ProjectFilename), TEXT("Saved"), TEXT("StagedBuilds"))
			);
	case EBuildType::PackagedBuild:
		return PlatformFile.ConvertToAbsolutePathForExternalAppForRead(
				*FPaths::Combine(FPaths::GetPath(ProjectFilename), TEXT("Saved"), TEXT("Packages"))
			);
	case EBuildType::Oplog:
		return FApp::GetZenStoreProjectIdForProject(ProjectFilename);
	}
	return PlatformFile.ConvertToAbsolutePathForExternalAppForRead(
				*FPaths::Combine(FPaths::GetPath(ProjectFilename), TEXT("Saved"), TEXT("DownloadedBuilds"))
			);
}

FString SBuildSelection::GetEffectiveDestination() const
{
	FString UserSelectedDestination = GetUserSelectedDestination();
	return UserSelectedDestination.IsEmpty() ? GetDefaultDestination() : UserSelectedDestination;
}

void SBuildSelection::SetSelectedStream(const TSharedPtr<FString> InSelectedStream)
{
	using namespace UE::BuildSelection::Internal;
	SelectedStream = InSelectedStream;
	WriteSetting(TEXT("Branch"), *SelectedStream);
}

void SBuildSelection::SetSelectedProject(const TSharedPtr<FString> InSelectedProject)
{
	using namespace UE::BuildSelection::Internal;
	SelectedProject = InSelectedProject;
	WriteSetting(TEXT("Project"), *SelectedProject);
}

void SBuildSelection::SetSelectedBuildType(const TSharedPtr<FString> InBuildType)
{
	using namespace UE::BuildSelection::Internal;
	SelectedBuildType = InBuildType;
	WriteSetting(TEXT("BuildType"), *SelectedBuildType);
}

void SBuildSelection::SetIncludePreflights(const bool bInIncludePreflights)
{
	using namespace UE::BuildSelection::Internal;
	bIncludePreflights = bInIncludePreflights;
	WriteSetting(TEXT("IncludePreflights"), bIncludePreflights ? TEXT("True") : TEXT("False"));
	PostViewFilterChange();
}

void SBuildSelection::SetUserSelectedProjectDictionaryRoot(const FStringView InRoot)
{
	if (InRoot.IsEmpty())
	{
		UserSelectedProjectDictionaryRootDir.Empty();
		UserSelectedProjectDictionary.Reset();
	}
	FString Root(InRoot);
	FString FullRoot = FPaths::ConvertRelativePathToFull(Root);
	if (!FPaths::IsSamePath(FullRoot, UserSelectedProjectDictionaryRootDir))
	{
		UserSelectedProjectDictionaryRootDir = FullRoot;
		UserSelectedProjectDictionary = MakeUnique<FUProjectDictionary>(UserSelectedProjectDictionaryRootDir);
	}
}

void SBuildSelection::SetFallbackProjectDictionaryRoot(const FStringView InRoot)
{
	if (InRoot.IsEmpty())
	{
		FallbackProjectDictionaryRootDir.Empty();
		FallbackProjectDictionary.Reset();
	}
	FString Root(InRoot);
	FString FullRoot = FPaths::ConvertRelativePathToFull(Root);
	if (!FPaths::IsSamePath(FullRoot, FallbackProjectDictionaryRootDir))
	{
		FallbackProjectDictionaryRootDir = FullRoot;
		FallbackProjectDictionary = MakeUnique<FUProjectDictionary>(FallbackProjectDictionaryRootDir);
	}
}

FString SBuildSelection::SanitizeForPath(const FString& InString)
{
	// TODO: Had to remove path sanitization as the engine only provides validation
	return InString;
}

FString SBuildSelection::SanitizeForZenId(const FString& InString)
{
	FString OutString = InString;
	for (int32 Index = 0; Index < OutString.Len(); ++Index)
	{
		if (!FChar::IsIdentifier(OutString[Index]) && OutString[Index] != TCHAR('.'))
		{
			OutString[Index] = TCHAR('_');
		}
	}

	// Trim leading and trailing underscores
	FString OutStringPreTrim = OutString;
	bool bCharsWereRemoved;
	do
	{
		OutString.TrimCharInline(TCHAR('_'), &bCharsWereRemoved);
	} while (bCharsWereRemoved);

	return OutString.IsEmpty() ? OutStringPreTrim : OutString;
}

FString SBuildSelection::SanitizeBranch(const FString& InString)
{
	FString OutString(InString);
	OutString.ReplaceCharInline(TCHAR('.'), TCHAR('-'));
	return SanitizeBucket(OutString);
}

FString SBuildSelection::SanitizeBucket(const FString& InString)
{
	TStringBuilder<64> OutputBuilder;
	for (int32 CharIndex = 0; CharIndex < InString.Len(); ++CharIndex)
	{
		TCHAR Character = FChar::ToLower(InString[CharIndex]);

		if (Character == TCHAR('.'))
		{
			if (OutputBuilder.Len() > 0)
			{
				if (OutputBuilder.LastChar() == TCHAR('-'))
				{
					OutputBuilder.RemoveSuffix(1);
				}
				OutputBuilder.AppendChar(Character);
			}
		}
		else if (FChar::IsIdentifier(Character))
		{
			OutputBuilder.AppendChar(Character);
		}
		else if (OutputBuilder.Len() > 0 &&
			OutputBuilder.LastChar() != TCHAR('-'))
		{
			OutputBuilder.AppendChar(TCHAR('-'));
		}
	}

	FString OutString = OutputBuilder.ToString();

	// Trim leading and trailing dashes
	bool bCharsWereRemoved;
	do
	{
		OutString.TrimCharInline(TEXT('-'), &bCharsWereRemoved);
	} while (bCharsWereRemoved);

	return OutString;
}

const TCHAR* SBuildSelection::LexToString(EBuildType BuildType)
{
	static const TCHAR* Strings[] =
	{
		TEXT("Oplog"),
		TEXT("StagedBuild"),
		TEXT("PackagedBuild"),
		TEXT("EditorPreCompiledBinary"),
		TEXT("EditorInstalledBuild"),
		TEXT("Unknown")
	};
	static_assert((int32)(EBuildType::Count) == UE_ARRAY_COUNT(Strings), "SBuildSelection::LexToString must contain a string for each member of EBuildType");

	return Strings[(int32)BuildType];
}

FReply SBuildSelection::ExploreDestination_OnClicked()
{
	if (GetSelectedBuildType() == EBuildType::Oplog)
	{
		UE::Zen::FZenLocalServiceRunContext RunContext;
		uint16 LocalPort = 8558;
		if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
		{
			if (!UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort))
			{
				UE::Zen::StartLocalService(RunContext);
				UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort);
			}
		}
		FPlatformProcess::LaunchURL(*FString::Printf(TEXT("http://localhost:%d/dashboard/?page=project&project=%s"), LocalPort, *GetEffectiveDestination()), nullptr, nullptr);
	}
	else
	{
		FPlatformProcess::ExploreFolder(*GetEffectiveDestination());
	}
	return FReply::Handled();
}

void SBuildSelection::DownloadSelectedArtifactsInSelectedBuildGroup()
{
	using namespace UE::Zen::Build;
	using namespace UE::BuildSelection::Internal;

	if (TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
	{
		TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
		if (SelectedItems.Num() == 0)
		{
			return;
		}

		const TSharedPtr<FString> SelectedConfiguration = BuildArtifactSelection->GetSelectedConfiguration();
		FName SelectedConfigurationName(NAME_None);
		if (BuildArtifactSelection->HasArtifactConfigurations() && SelectedConfiguration && *SelectedConfiguration != TEXT("All"))
		{
			SelectedConfigurationName = FName(*SelectedConfiguration);
		}
		for (const FString& ArtifactForSelectedGroup : BuildArtifactSelection->GetSelectedArtifacts())
		{
			if (FArtifact* Artifact = SelectedItems[0]->NamedArtifacts.Find(ArtifactForSelectedGroup))
			{
				if ((SelectedConfigurationName != NAME_None) && !Artifact->Configurations.Contains(SelectedConfigurationName))
				{
					continue;
				}

				DownloadItem(*SelectedItems[0], ArtifactForSelectedGroup, *Artifact);
			}
		}
	}
}

bool SBuildSelection::DownloadItem(const FBuildGroup& Group, const FString& ArtifactName, const UE::BuildSelection::Internal::FArtifact& Artifact, FString&& DownloadSpecJSONContents, TArray<FString>&& PartNames)
{
	using namespace UE::Zen::Build;

	TSharedPtr<FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get();
	if (!ServiceInstance)
	{
		return false;
	}

	FString DestinationPlatformName = ArtifactName;
	if (FCbFieldView CookPlatformField = Artifact.BuildRecord.Metadata["cookPlatform"]; CookPlatformField.HasValue() && !CookPlatformField.HasError())
	{
		DestinationPlatformName = *WriteToString<64>(CookPlatformField.AsString());
	}

	FName TargetPlatformName = NAME_None;
	if (FCbFieldView PlatformField = Artifact.BuildRecord.Metadata["platform"]; PlatformField.HasValue() && !PlatformField.HasError())
	{
		TargetPlatformName = FName(PlatformField.AsString());
	}

	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;
	const bool bAppendBuildNameToDestination = bAppendBuildNameToDestinations[BuildTypeIndex];

	FString TransferName = FString::Printf(TEXT("%s-%s"), *Group.DisplayName, *DestinationPlatformName);

	if (BuildType == EBuildType::Oplog)
	{
		FString RootDir = FPaths::RootDir();
		FString EngineDir = FPaths::EngineDir();
		FString ProjectFilename = FUProjectDictionary::GetDefault().GetProjectPathForGame(**SelectedProject);

		if (ProjectFilename.IsEmpty() && UserSelectedProjectDictionary)
		{
			RootDir = UserSelectedProjectDictionaryRootDir;
			EngineDir = FPaths::Combine(RootDir, TEXT("Engine"));
			ProjectFilename = UserSelectedProjectDictionary->GetProjectPathForGame(**SelectedProject);
		}

		if (ProjectFilename.IsEmpty() && FallbackProjectDictionary)
		{
			RootDir = FallbackProjectDictionaryRootDir;
			EngineDir = FPaths::Combine(RootDir, TEXT("Engine"));
			ProjectFilename = FallbackProjectDictionary->GetProjectPathForGame(**SelectedProject);
		}
		FString DestinationProjectId = GetEffectiveDestination();
		FString DestinationOplogId = DestinationPlatformName;
		if (bAppendBuildNameToDestination)
		{
			DestinationOplogId = FString::Printf(TEXT("%s.%s"), *DestinationOplogId, *SanitizeForZenId(Group.DisplayName));
		}

		FString HostOverride;
		EBuildTransferRequestFlags RequestFlags = EBuildTransferRequestFlags::Clean;
		PreOplogBuildTransfer.ExecuteIfBound(HostOverride, RequestFlags);

		FBuildServiceInstance::FBuildTransfer BuildTransfer =
			ServiceInstance->StartOplogBuildTransfer(
				Artifact.BuildRecord.BuildId,
				TransferName,
				DestinationProjectId,
				DestinationOplogId,
				RootDir,
				EngineDir,
				ProjectFilename,
				Group.Namespace,
				Artifact.BuildRecord.BucketId,
				TargetPlatformName,
				HostOverride,
				RequestFlags);
		OnBuildTransferStarted.ExecuteIfBound(BuildTransfer, Group.DisplayName, ArtifactName);
	}
	else
	{
		FString DestinationFolder;
		if (bAppendBuildNameToDestination)
		{
			DestinationFolder = FPaths::Combine(GetEffectiveDestination(), SanitizeForPath(Group.DisplayName), DestinationPlatformName);
		}
		else
		{
			DestinationFolder = FPaths::Combine(GetEffectiveDestination(), DestinationPlatformName);
		}

		FString HostOverride;
		EBuildTransferRequestFlags RequestFlags = EBuildTransferRequestFlags::Scavenge;
		PreBuildTransfer.ExecuteIfBound(HostOverride, RequestFlags);

		FBuildServiceInstance::FBuildTransfer BuildTransfer =
			ServiceInstance->StartBuildTransfer(
				Artifact.BuildRecord.BuildId,
				TransferName,
				DestinationFolder,
				Group.Namespace,
				Artifact.BuildRecord.BucketId,
				DownloadSpecJSONContents.IsEmpty() ? UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter : FString(),
				DownloadSpecJSONContents.IsEmpty() ? UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter : FString(),
				MoveTemp(DownloadSpecJSONContents),
				TArray<FString>(),
				MoveTemp(PartNames),
				HostOverride,
				RequestFlags);
		FString BuildTransferName = FString::Printf(TEXT("%s%s"), *Group.DisplayName, DownloadSpecJSONContents.IsEmpty() ? TEXT("") : TEXT("*"));
		OnBuildTransferStarted.ExecuteIfBound(BuildTransfer, BuildTransferName, ArtifactName);
	}

	return true;
}

void SBuildSelection::OnChooseDestinationDirectoryClicked()
{
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString Title = LOCTEXT("BuildSelection_DestinationDirectoryBrowserTitle", "Choose destination directory").ToString();

		FString NewDestination = GetUserSelectedDestination();
		if (DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			Title,
			GetEffectiveDestination(),
			NewDestination))
		{
			SetUserSelectedDestination(NewDestination);
		}
	}
}

void SBuildSelection::OnChooseTransferFilterClicked()
{
	if (!FDialogCommands::IsRegistered())
	{
		FDialogCommands::Register();
	}

	EBuildType BuildType = GetSelectedBuildType();
	int BuildTypeIndex = (int)BuildType;

	TSharedRef<SBuildTransferFilters> TransferFilters = SNew(SBuildTransferFilters)
		.IncludeFilter(MakeShared<FString>(UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter))
		.ExcludeFilter(MakeShared<FString>(UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter));

	TSharedRef<SCustomDialog> TransferFilterDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("BuildTransferFiltersHeader", "Build Transfer Filter")))
		.Content()
		[
			TransferFilters
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("OK", "OK")),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
	});

	if (TransferFilterDialog->ShowModal() == 0)
	{
		SetUserSelectedTransferFilter(TransferFilters->GetIncludeFilter(), TransferFilters->GetExcludeFilter());
	}
}

TSharedRef<SWidget> SBuildSelection::GetBuildDestinationPanel()
{
	TSharedPtr<SCheckBox> AppendBuildNameCheckbox;

	return
	SNew(SVerticalBox)

	+ SVerticalBox::Slot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)
		.Visibility_Lambda([this]()
		{
			TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
			return (SelectedItems.Num() == 0) || GetSelectedBuildType() == EBuildType::Oplog ? EVisibility::Collapsed : EVisibility::Visible;
		})
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SBox)
				.MaxDesiredWidth(500.0f)
				[
					SNew(SEditableTextBox)
					.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
					.MinDesiredWidth(200.0f)
					.HintText_Lambda([this]()
					{
						return FText::FromString(GetDefaultDestination());
					})
					.Text_Lambda([this]()
					{
						return FText::FromString(GetUserSelectedDestination());
					})
					.OnTextCommitted_Lambda([this](const FText& Text, const ETextCommit::Type CommitType)
					{
						SetUserSelectedDestination(Text.ToString());
					})
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SEditableTextBox)
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.MinDesiredWidth(200.0f)
				.IsReadOnly(true)
				.Text_Lambda([this]()
				{
					EBuildType BuildType = GetSelectedBuildType();
					int BuildTypeIndex = (int)BuildType;
					if (!UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter.IsEmpty())
					{
						return FText::Format(LOCTEXT("BuildTransferFilter_IncludeAndExcludeFormat", "Incl: {0}, Excl: {1}"),
							FText::FromString(UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter),
							FText::FromString(UserSelectedTransferFilters[BuildTypeIndex].ExcludeFilter));
					}
					if (!UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter.IsEmpty())
					{
						return FText::FromString(UserSelectedTransferFilters[BuildTypeIndex].IncludeFilter);
					}
					return LOCTEXT("BuildTransferFilter_DownloadAllFiles", "<All files>");
				})
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SButton)
				.OnClicked_Lambda([this]()
				{
					OnChooseDestinationDirectoryClicked();
					return FReply::Handled();
				})
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Zen.BrowseContent"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("BuildSelection_ChooseTransferFilterTooltip", "Change what gets downloaded within builds."))
				.OnClicked_Lambda([this]()
				{
					OnChooseTransferFilterClicked();
					return FReply::Handled();
				})
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Zen.Filter"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)
		.Visibility_Lambda([this]()
		{
			TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
			return (SelectedItems.Num() == 0) || GetSelectedBuildType() != EBuildType::Oplog ? EVisibility::Collapsed : EVisibility::Visible;
		})
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SEditableTextBox)
			.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
			.MinDesiredWidth(200.0f)
			.HintText_Lambda([this]()
			{
				return FText::FromString(GetDefaultDestination());
			})
			.Text_Lambda([this]()
			{
				return FText::FromString(GetUserSelectedDestination());
			})
			.OnTextCommitted_Lambda([this](const FText& Text, const ETextCommit::Type CommitType)
			{
				SetUserSelectedDestination(Text.ToString());
			})
		]
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(AppendBuildNameCheckbox, SCheckBox)
			.IsChecked_Lambda([this]
			{
				EBuildType BuildType = GetSelectedBuildType();
				int BuildTypeIndex = (int)BuildType;
				return bAppendBuildNameToDestinations[BuildTypeIndex] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState)
			{
				EBuildType BuildType = GetSelectedBuildType();
				int BuildTypeIndex = (int)BuildType;
				if (InNewState == ECheckBoxState::Checked)
				{
					SetUserSelectedAppendBuildNameToDestination(true);
				}
				else if (InNewState == ECheckBoxState::Unchecked)
				{
					SetUserSelectedAppendBuildNameToDestination(false);
				}
			})
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
			.IsFocusable(false)
			.OnClicked_Lambda([this, AppendBuildNameCheckbox]()
				{
					EBuildType BuildType = GetSelectedBuildType();
					int BuildTypeIndex = (int)BuildType;
					ECheckBoxState NewState = ECheckBoxState::Checked;
					if (AppendBuildNameCheckbox->IsChecked())
					{
						NewState = ECheckBoxState::Unchecked;
					}

					if (NewState == ECheckBoxState::Checked)
					{
						SetUserSelectedAppendBuildNameToDestination(true);
					}
					else if (NewState == ECheckBoxState::Unchecked)
					{
						SetUserSelectedAppendBuildNameToDestination(false);
					}
					return FReply::Handled();
				})
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Left)
				.Text(LOCTEXT("BuildSelection_DestinationAppendBuildName", "Append build name"))
			]
		]
	];
}

TSharedRef<SWidget> SBuildSelection::GetGridPanel()
{
	using namespace UE::BuildSelection::Internal;

	TSharedRef<SVerticalBox> Panel =
	SNew(SVerticalBox)
	.IsEnabled_Lambda([this]
	{
		if (TSharedPtr<UE::Zen::Build::FBuildServiceInstance> ServiceInstance = BuildServiceInstance.Get())
		{
			return ServiceInstance->GetConnectionState() == UE::Zen::Build::FBuildServiceInstance::EConnectionState::ConnectionSucceeded && !ServiceInstance->GetNamespacesAndBuckets().IsEmpty();
		}
		return false;
	});

	const float MinDesiredWidth = 50.0f;

	const float RowMargin = 2.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	Panel->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)

		// Stream
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_Stream", "Stream"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SAssignNew(StreamWidget, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&StreamList)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
				{
					if (Item.IsValid() && *Item != *SelectedStream)
					{
						SetSelectedStream(Item);
						RebuildLists();
					}
				})
				.OnGenerateWidget(this, &SBuildSelection::OnGenerateTextBlockFromString)
				[
					SNew(STextBlock)
						.MinDesiredWidth(MinDesiredWidth)
						.Text_Lambda([this]()
						{
							return FText::FromString(SelectedStream ? **SelectedStream : TEXT(""));
						})
				]
			]
		]

		// Project
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_Project", "Project"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SAssignNew(ProjectWidget, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&ProjectList)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
				{
					if (Item.IsValid() && *Item != *SelectedProject)
					{
						SetSelectedProject(Item);
						RebuildLists();
					}
				})
				.OnGenerateWidget(this, &SBuildSelection::OnGenerateTextBlockFromString)
				[
					SNew(STextBlock)
						.MinDesiredWidth(MinDesiredWidth)
						.Text_Lambda([this]()
						{
							return FText::FromString(SelectedProject ? **SelectedProject : TEXT(""));
						})
				]
			]
		]

		// Build Type
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_BuildType", "Build Type"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SAssignNew(BuildTypeWidget, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&BuildTypeList)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
				{
					if (Item.IsValid() && *Item != *SelectedBuildType)
					{
						SetSelectedBuildType(Item);
						RebuildLists();
					}
				})
				.OnGenerateWidget(this, &SBuildSelection::OnGenerateBuildTypeTextBlockFromString)
				[
					SNew(STextBlock)
						.MinDesiredWidth(MinDesiredWidth)
						.Text_Lambda([this]()
						{
							if (!SelectedBuildType)
							{
								return FText::GetEmpty();
							}
							return ConvertBuildTypeToText(*SelectedBuildType);
						})
				]
			]
		]

		// Text Filter
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_TextFilter", "Text Filter"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SEditableTextBox)
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.Text_Lambda([this]()
				{
					return FText::FromString(SelectedTextFilter);
				})
				.OnTextChanged_Lambda([this](const FText& Text)
				{
					SelectedTextFilter = Text.ToString();
					PostViewFilterChange();
				})
				.OnTextCommitted_Lambda([this](const FText& Text, const ETextCommit::Type CommitType)
				{
					SelectedTextFilter = Text.ToString();
					PostViewFilterChange();
				})
			]
		]

		// Required Platforms
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_PlatformFilters", "Platforms"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SAssignNew(PlatformFiltersWidget, SMultiSelectComboBox)
				.SelectValues(&PlatformList)
				.OnCheckedValuesChanged_Lambda([this]()
				{
					RegenerateActivePlatformFilters();
					PostViewFilterChange();
				})
			]
		]

		// Include Preflights
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.VAlign(VAlign_Top)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Text(LOCTEXT("BuildSelection_ShowPreflights", "Preflights"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(ColumnMargin, RowMargin))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return bIncludePreflights ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState)
				{
					if (InNewState == ECheckBoxState::Checked)
					{
						SetIncludePreflights(true);
					}
					else if (InNewState == ECheckBoxState::Unchecked)
					{
						SetIncludePreflights(false);
					}
				})
			]
		]
	];

	Panel->AddSlot()
	.Padding(FMargin(ColumnMargin, 10, ColumnMargin, 0))
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(STextBlock)
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("BuildSelection_BuildsLabel", "Builds"))
	];

	Panel->AddSlot()
	.Padding(FMargin(ColumnMargin, RowMargin))
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(BuildGroupListView, SListView<FBuildSelectionBuildGroupPtr>)
				.ListItemsSource(&PresentedBuildGroups)
				.OnGenerateRow(this, &SBuildSelection::GenerateBuildGroupRow)
				.OnSelectionChanged(this, &SBuildSelection::BuildGroupSelectionChanged)
				.OnContextMenuOpening(this, &SBuildSelection::OnGetBuildGroupContextMenuContent)
				.SelectionMode(ESelectionMode::Single)
				.OnIsSelectableOrNavigable(this, &SBuildSelection::BuildGroupIsSelectableOrNavigable)
				.IsEnabled_Lambda([this]
				{
					return !BuildListRefreshesInProgress;
				})
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(FBuildGroupIds::ColName).DefaultLabel(LOCTEXT("BuildSelection_BuildGroupColName", "Name"))
						.FillWidth(0.4f)
						.SortMode(this, &SBuildSelection::GetColumnSortMode, FBuildGroupIds::ColName)
						.OnSort(this, &SBuildSelection::OnColumnSortModeChanged)
					+ SHeaderRow::Column(FBuildGroupIds::ColCommit).DefaultLabel(LOCTEXT("BuildSelection_BuildGroupColCommit", "Commit"))
						.DefaultTooltip(LOCTEXT("BuildSelection_BuildGroupColCommitTooltip", "Commit/Changelist for the build"))
						.FillWidth(0.10f).HAlignCell(HAlign_Center).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
						.SortMode(this, &SBuildSelection::GetColumnSortMode, FBuildGroupIds::ColCommit)
						.OnSort(this, &SBuildSelection::OnColumnSortModeChanged)
					+ SHeaderRow::Column(FBuildGroupIds::ColCategory).DefaultLabel(LOCTEXT("BuildSelection_BuildGroupColCategory", "Category"))
						.DefaultTooltip(LOCTEXT("BuildSelection_BuildGroupColCategoryTooltip", "Category for the build"))
						.SortMode(this, &SBuildSelection::GetColumnSortMode, FBuildGroupIds::ColCategory)
						.OnSort(this, &SBuildSelection::OnColumnSortModeChanged)
						.FillWidth(0.25f).HAlignCell(HAlign_Left).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
					+ SHeaderRow::Column(FBuildGroupIds::ColCreated).DefaultLabel(LOCTEXT("BuildSelection_BuildGroupColCreated", "Created"))
						.DefaultTooltip(LOCTEXT("BuildSelection_BuildGroupColCreatedTooltip", "When the build was created"))
						.FillWidth(0.15f).HAlignCell(HAlign_Left).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
						.SortMode(this, &SBuildSelection::GetColumnSortMode, FBuildGroupIds::ColCreated)
						.OnSort(this, &SBuildSelection::OnColumnSortModeChanged)
				)
		]
	];

	Panel->AddSlot()
	.Padding(FMargin(ColumnMargin, 3, ColumnMargin, 0))
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Bottom)
	[
		SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			.Text_Lambda([this]()
			{
				if (BuildListRefreshesInProgress)
				{
					return LOCTEXT("BuildSelection_ResultLoading", "Loading...");
				}
				return FText::Format(LOCTEXT("BuildSelection_ResultDescription", "{0} {0}|plural(one=item,other=items)"), FText::AsNumber(PresentedBuildGroups.Num()));
			})
	];

	Panel->AddSlot()
	.Padding(FMargin(ColumnMargin, RowMargin))
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Bottom)
	[
		SNew(SHorizontalBox)
		.Visibility_Lambda([this]()
		{
			return !!BuildListRefreshesInProgress || BuildGroupListView->GetNumItemsSelected() == 0 ? EVisibility::Collapsed : EVisibility::Visible;
		})
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			SNew(SScrollBox)
			.Orientation(EOrientation::Orient_Horizontal)
			+ SScrollBox::Slot()
			.FillSize(1.0f)
			[
				SAssignNew(BuildArtifactSelection, SBuildArtifactSelection)
					.BuildServiceInstance(BuildServiceInstance)
					.OnDownloadWithSpec_Lambda([this](const FString& ArtifactName, const UE::BuildSelection::Internal::FArtifact& Artifact, const FString& Spec, const TArray<FString>& PartNames)
					{
						TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
						if (SelectedItems.Num() > 0)
						{
							DownloadItem(*SelectedItems[0], ArtifactName, Artifact, FString(Spec), TArray<FString>(PartNames));
						}
					})
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				[
					SNew(STextBlock)
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Text(LOCTEXT("BuildSelection_Destination", "Destination"))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
					.ToolTipText(LOCTEXT("BuildSelection_DestinationExploreTooltip", "Explore the destination"))
					.OnClicked(this, &SBuildSelection::ExploreDestination_OnClicked)
					[
						SNew(SImage)
						.Image_Lambda([this]
						{
							return GetSelectedBuildType() == EBuildType::Oplog ? FAppStyle::Get().GetBrush("Zen.BrowserView") : FAppStyle::Get().GetBrush("Zen.FolderView");
						})
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				GetBuildDestinationPanel()
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(0, RowMargin))
			[
				SNew(SButton)
				.Text(LOCTEXT("BuildSelection_Download", "Download"))
				.ToolTipText(LOCTEXT("BuildSelection_DownloadTooltip", "Start a download of the selected build for the selected platforms"))
				.ButtonStyle(FAppStyle::Get(), "Button")
				.IsEnabled_Lambda([this]()
				{
					const bool bDestinationValid = !GetEffectiveDestination().IsEmpty();
					if (!bDestinationValid)
					{
						return false;
					}

					TArray<FBuildSelectionBuildGroupPtr> SelectedItems = BuildGroupListView->GetSelectedItems();
					if (SelectedItems.Num() == 0)
					{
						return false;
					}

					const TSharedPtr<FString> SelectedConfiguration = BuildArtifactSelection->GetSelectedConfiguration();
					FName SelectedConfigurationName(NAME_None);
					if (BuildArtifactSelection->HasArtifactConfigurations() && SelectedConfiguration && *SelectedConfiguration != TEXT("All"))
					{
						SelectedConfigurationName = FName(*SelectedConfiguration);
					}
					for (const FString& ArtifactForSelectedGroup : BuildArtifactSelection->GetSelectedArtifacts())
					{
						if (FArtifact* FoundArtifact = SelectedItems[0]->NamedArtifacts.Find(ArtifactForSelectedGroup))
						{
							if ((SelectedConfigurationName == NAME_None) || FoundArtifact->Configurations.Contains(SelectedConfigurationName))
							{
								return true;
							}
						}
					}
					return false;
				})
				.OnClicked_Lambda([this]()
				{
					DownloadSelectedArtifactsInSelectedBuildGroup();
					return FReply::Handled();
				})
			]
		]
	];

	return Panel;
}

void
SBuildGroupTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FBuildSelectionBuildGroupPtr InBuildGroup)
{
	BuildGroup = InBuildGroup;

	SMultiColumnTableRow<FBuildSelectionBuildGroupPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget>
SBuildGroupTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace UE::BuildSelection::Internal;

	if (ColumnName == FBuildGroupIds::ColName)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock).Text(FText::FromString(BuildGroup->DisplayName))
				];
	}
	else if (ColumnName == FBuildGroupIds::ColCommit)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock).Text(FText::FromString(BuildGroup->CommitIdentifier))
				];
	}
	else if (ColumnName == FBuildGroupIds::ColCategory)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock).Text(FText::FromString(BuildGroup->Category))
				];
	}
	else if (ColumnName == FBuildGroupIds::ColCreated)
	{
		if (BuildGroup->CreatedAt.GetTicks() != 0)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f,1.f)
				[
					SNew(STextBlock).Text(FText::AsDateTime(BuildGroup->CreatedAt, EDateTimeStyle::Short, EDateTimeStyle::Short, FString(), FInternationalization::Get().GetCulture(FPlatformMisc::GetDefaultLocale())))
				];
		}
	}

	return SNullWidget::NullWidget;
}

const FSlateBrush*
SBuildGroupTableRow::GetBorder() const
{
	return STableRow<FBuildSelectionBuildGroupPtr>::GetBorder();
}

FReply
SBuildGroupTableRow::OnBrowseClicked()
{
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
