// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBuildTransferFilters.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StorageServerBuild"

void SBuildTransferFilters::Construct(const FArguments& InArgs)
{
	if (InArgs._IncludeFilter.Get())
	{
		IncludeFilter = *InArgs._IncludeFilter.Get();
	}
	if (InArgs._ExcludeFilter.Get())
	{
		ExcludeFilter = *InArgs._ExcludeFilter.Get();
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];
}

TSharedRef<SWidget> SBuildTransferFilters::GetGridPanel()
{
	TSharedRef<SGridPanel> Panel = SNew(SGridPanel)
		.FillColumn(1, 1.0f);

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	Panel->AddSlot(0, 0)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("BuildTransferFilters_IncludeFilterLabel", "Include Filter:"))
	];

	Panel->AddSlot(1, 0)
	[
		SNew(SEditableTextBox)
		.MinDesiredWidth(400.0f)
		.Text(FText::FromString(IncludeFilter))
		.OnTextChanged_Lambda([this](const FText& Text)
		{
			IncludeFilter = Text.ToString();
		})
		.OnTextCommitted_Lambda([this](const FText& Text, const ETextCommit::Type CommitType)
		{
			IncludeFilter = Text.ToString();
		})
	];

	Panel->AddSlot(0, 1)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("BuildTransferFilters_ExcludeFilterLabel", "Exclude Filter:"))
	];

	Panel->AddSlot(1, 1)
	[
		SNew(SEditableTextBox)
		.MinDesiredWidth(400.0f)
		.Text(FText::FromString(ExcludeFilter))
		.OnTextChanged_Lambda([this](const FText& Text)
		{
			ExcludeFilter = Text.ToString();
		})
		.OnTextCommitted_Lambda([this](const FText& Text, const ETextCommit::Type CommitType)
		{
			ExcludeFilter = Text.ToString();
		})
	];

	Panel->AddSlot(0, 2)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, 10.0f))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("BuildTransferFilters_NotesLabel", "Notes:"))
	];

	Panel->AddSlot(1, 2)
	[
		SNew(STextBlock)
		.Margin(FMargin(0.0f, 10.0f))
		.Text(LOCTEXT("BuildTransferFilters_NotesContents",
			"Windows style wildcard(s) (using * and ?) to match file paths to exclude, separated by semicolon(;).\n"
			"Specifying a non-empty inclusion limits the download to only files that match the inclusion patterns.\n"
			"Exclusions override any inclusions.\n\n"
			"Examples:\n"
			"Include filter to download Windows executables only: *.exe;*.dll\n"
			"Exclude filter to not download symbol files: *.pdb*;*.map"
			))
	];

	return Panel;
}

#undef LOCTEXT_NAMESPACE
