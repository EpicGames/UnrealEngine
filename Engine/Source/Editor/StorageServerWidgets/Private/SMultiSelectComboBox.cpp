// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiSelectComboBox.h"

#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "StorageServerBuild"

void SMultiSelectComboBox::Construct(const FArguments& InArgs)
{
	using namespace UE::Slate::Containers;
	OnCheckedValuesChanged = InArgs._OnCheckedValuesChanged;
	SelectValues = InArgs._SelectValues;

	TSharedPtr<SHorizontalBox> HorizontalBox;
	
	ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
		];

	HorizontalBox->AddSlot()
		.HAlign(HAlign_Fill)
		[
			SNew(SBox)
			.MinDesiredWidth(70.0f)
			[
				SNew(SComboButton)
					.IsEnabled_Lambda([this]()
						{
							return SelectValues->Num() > 0;
						})
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
						{
							bool bFirst = true;
							TStringBuilder<64> StringBuilder;
							for (const FString& CheckedValue : CheckedValues)
							{
								bool bInSelectValues = SelectValues->ContainsByPredicate([&CheckedValue](const TSharedPtr<FString>& A)
								{
									return *A == CheckedValue;
								});
								if (bInSelectValues)
								{
									if (!bFirst)
									{
										StringBuilder << TEXT(", ");
									}
									bFirst = false;
									StringBuilder << CheckedValue;
								}
							}
							return StringBuilder.Len() == 0 ? LOCTEXT("MultiSelectComboBox_AnyValue", "Any") : FText::FromString(*StringBuilder);
						})
				]
				.OnGetMenuContent(FOnGetContent::CreateSP(this, &SMultiSelectComboBox::GenerateMenuItems))
			]
		];
}

SMultiSelectComboBox::~SMultiSelectComboBox()
{
}

bool SMultiSelectComboBox::IsChecked(FStringView Value) const
{
	return CheckedValues.Contains(Value);
}

TSharedRef<class SWidget> SMultiSelectComboBox::GenerateMenuItems()
{
	TSharedRef<SVerticalBox> Contents = SNew(SVerticalBox);

	for (int i = 0; i < this->SelectValues->Num(); i++)
	{
		Contents->AddSlot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this, i]() { return CheckedValues.Contains(*this->SelectValues->operator[](i)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this, i](ECheckBoxState InNewState)
					{
						 OnSelectedChangedFromMultiselect(SelectValues->operator[](i));
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
					.IsFocusable(false)			

					.OnClicked_Lambda([this, i]() { OnSelectedChangedFromMultiselect(SelectValues->operator[](i)); return FReply::Handled(); })
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Left)
						.MinDesiredWidth(60)
						.Text(FText::FromString(*this->SelectValues->operator[](i)))
					]
				]
			];
	}

	return Contents;
}

void SMultiSelectComboBox::OnSelectedChangedFromMultiselect(TSharedPtr<FString> Value)
{
	if (Value->IsEmpty())
	{
		return;
	}

	if(!CheckedValues.Contains(*Value))
	{
		CheckedValues.Add(FString(*Value));
		OnCheckedValuesChanged.Execute();
	}
	else
	{
		bool bAnyRemoved = false;
		while (CheckedValues.Contains(*Value))
		{
			CheckedValues.Remove(FString(*Value));
			bAnyRemoved = true;
		}

		if (bAnyRemoved)
		{
			OnCheckedValuesChanged.Execute();
		}
	}
}


#undef LOCTEXT_NAMESPACE