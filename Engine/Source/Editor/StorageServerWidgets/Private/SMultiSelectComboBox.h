// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Containers/ObservableArray.h"

DECLARE_DELEGATE(FOnMultiSelectComboBoxCheckedValuesChanged)

class SMultiSelectComboBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMultiSelectComboBox) {}
		SLATE_ARGUMENT(TArray<TSharedPtr<FString>>*, SelectValues)
		SLATE_EVENT(FOnMultiSelectComboBoxCheckedValuesChanged, OnCheckedValuesChanged)
	SLATE_END_ARGS()

	virtual ~SMultiSelectComboBox();

	void Construct(const FArguments& InArgs);

	bool IsChecked(FStringView Value) const;

private:
	TSharedRef<class SWidget> GenerateMenuItems();

	void OnSelectedChangedFromMultiselect(TSharedPtr<FString> Value);

	FOnMultiSelectComboBoxCheckedValuesChanged OnCheckedValuesChanged;
	TArray<FString> CheckedValues;
	TArray<TSharedPtr<FString>>* SelectValues;
};