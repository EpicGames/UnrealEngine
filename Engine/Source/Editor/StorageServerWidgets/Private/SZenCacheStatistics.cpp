// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenCacheStatistics.h"
#include "Experimental/ZenServerInterface.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ZenDashboard"

void SZenCacheStatistics::Construct(const FArguments& InArgs)
{
	ZenServiceInstance = InArgs._ZenServiceInstance;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SZenCacheStatistics::UpdateGridPanels));
}

EActiveTimerReturnType SZenCacheStatistics::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*GridSlot)
	[
		GetGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SZenCacheStatistics::GetGridPanel()
{
	TSharedRef<SGridPanel> Panel = SNew(SGridPanel);

	bool bHaveStats = false;
	UE::Zen::FZenCacheStats ZenStats;

	if (TSharedPtr<UE::Zen::FZenServiceInstance> ServiceInstance = ZenServiceInstance.Get())
	{
		if (ServiceInstance->GetCacheStats(ZenStats))
		{
			bHaveStats = true;
		}
	}

	const static FNumberFormattingOptions SingleDecimalFormatting = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);

	int32 Row = 0;

	const float RowMargin = 0.0f;
	const float TitleMargin = 10.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);


	Panel->AddSlot(0, Row)
	.ColumnSpan(2)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Cache", "Local Cache"))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheHitRate", "Hit Rate"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([CacheStats = ZenStats.General]
		{
			int64 CacheTotal = CacheStats.Hits + CacheStats.Misses;
			return CacheTotal == 0 ? LOCTEXT("CacheNoHitRateValue", "-") : FText::AsPercent(CacheStats.HitRatio, &SingleDecimalFormatting);
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheHitQuantity", "Hits"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, CacheHits = ZenStats.General.Hits]
		{
			return bHaveStats ? FText::AsNumber(CacheHits) : LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheMissQuantity", "Misses"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, CacheMisses = ZenStats.General.Misses]
		{
			return bHaveStats ? FText::AsNumber(CacheMisses) : LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheWriteQuantity", "Writes"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, CacheWrites = ZenStats.General.Writes]
		{
			return bHaveStats ? FText::AsNumber(CacheWrites) : LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheRequests", "Requests"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, Requests = ZenStats.Request.Count]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(Requests);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("CacheBadRequests", "Bad Requests"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, BadRequests = ZenStats.General.BadRequestCount]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(BadRequests);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	if (!ZenStats.Upstream.EndPoint.IsEmpty())
	{
		Panel->AddSlot(0, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, TitleMargin, ColumnMargin, 0.0f))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("UpstreamServer", "Upstream Server"))
		];

		Panel->AddSlot(1, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, TitleMargin, ColumnMargin, 0.0f))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("HitRate", "Hit Rate"))
		];

		Panel->AddSlot(2, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, TitleMargin, ColumnMargin, 0.0f))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("Downloaded", "Downloaded"))
		];

		Panel->AddSlot(3, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, TitleMargin, ColumnMargin, 0.0f))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("Uploaded", "Uploaded"))
		];

		Panel->AddSlot(4, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, TitleMargin, ColumnMargin, 0.0f))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Text(LOCTEXT("URL", "URL"))
		];
		Row++;

		for (const UE::Zen::FZenCacheStats::FEndPointStats& EndpointStats : ZenStats.Upstream.EndPoint)
		{
			Panel->AddSlot(0, Row)
			[
				SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.Text_Lambda([EndpointStats] { return FText::FromString(EndpointStats.Name); })
			];

			Panel->AddSlot(1, Row)
			[
				SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.Text_Lambda([EndpointStats] { return FText::AsPercent(EndpointStats.HitRatio, &SingleDecimalFormatting); })
			];

			Panel->AddSlot(2, Row)
			[
				SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.Text_Lambda([DownloadedMB = EndpointStats.DownloadedMB]
					{
						uint64 DownloadedBytes = uint64(DownloadedMB*1024*1024);
						return FText::AsMemory(DownloadedBytes, (DownloadedBytes > 1024) ? &SingleDecimalFormatting : nullptr, nullptr, EMemoryUnitStandard::IEC);
					})
												
			];

			Panel->AddSlot(3, Row)
			[
				SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.Text_Lambda([UploadedMB = EndpointStats.UploadedMB]
					{
						uint64 UploadedBytes = uint64(UploadedMB*1024*1024);
						return FText::AsMemory(UploadedBytes, (UploadedBytes > 1024) ? &SingleDecimalFormatting : nullptr, nullptr, EMemoryUnitStandard::IEC);
					})
			];

			Panel->AddSlot(4, Row)
			[
				SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.Text_Lambda([EndpointStats] { return FText::FromString(EndpointStats.Url); })
			];

			Row++;
		}
	}

	return Panel;
}

#undef LOCTEXT_NAMESPACE
