// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenDialogs.h"

#include "ZenServiceInstanceManager.h"
#include "SZenCacheStatistics.h"
#include "SZenCidStoreStatistics.h"
#include "SZenProjectStatistics.h"
#include "SZenServiceStatus.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ZenEditor"

void SZenStoreStausDialog::Construct(const FArguments& InArgs)
{
	TSharedPtr<UE::Zen::FServiceInstanceManager> ServiceInstanceManager = MakeShared<UE::Zen::FServiceInstanceManager>();

	this->ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 10.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(SZenServiceStatus)
						.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
				]
				// Stats panel
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 10.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenCacheStatistics)
								.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenProjectStatistics)
								.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenCidStoreStatistics)
								.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]
				]
		];
}

#undef LOCTEXT_NAMESPACE
