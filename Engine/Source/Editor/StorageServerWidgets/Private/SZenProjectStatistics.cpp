// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenProjectStatistics.h"
#include "Experimental/ZenServerInterface.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ZenDashboard"

void SZenProjectStatistics::Construct(const FArguments& InArgs)
{
	ZenServiceInstance = InArgs._ZenServiceInstance;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SZenProjectStatistics::UpdateGridPanels));
}

EActiveTimerReturnType SZenProjectStatistics::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*GridSlot)
	[
		GetGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SZenProjectStatistics::GetGridPanel()
{
	TSharedRef<SGridPanel> Panel = SNew(SGridPanel);

	bool bHaveStats = false;
	UE::Zen::FZenProjectStats ZenStats;

	if (TSharedPtr<UE::Zen::FZenServiceInstance> ServiceInstance = ZenServiceInstance.Get())
	{
		if (ServiceInstance->GetProjectStats(ZenStats))
		{
			bHaveStats = true;
		}
	}

	int64 TotalHits = ZenStats.General.Op.HitCount + ZenStats.General.Chunk.HitCount;
	int64 TotalMisses = ZenStats.General.Op.MissCount + ZenStats.General.Chunk.MissCount;
	int64 TotalWrites = ZenStats.General.Op.WriteCount + ZenStats.General.Chunk.WriteCount;

	const static FNumberFormattingOptions SingleDecimalFormatting = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);

	int32 Row = 0;

	const float RowMargin = 0.0f;
	const float TitleMargin = 10.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);


	Panel->AddSlot(0, Row)
	.ColumnSpan(2)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("CAS", "Local Project Store"))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectHitRate", "Hit Rate"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, TotalHits, TotalMisses]
		{
			if (bHaveStats && ((TotalHits+TotalMisses) > 0))
			{
				double HitRate = static_cast<double>(TotalHits) / static_cast<double>(TotalHits + TotalMisses);
				return FText::AsPercent(HitRate, &SingleDecimalFormatting);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectHitQuantity", "Hits"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, TotalHits]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(TotalHits);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectMissQuantity", "Misses"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, TotalMisses]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(TotalMisses);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectWriteQuantity", "Writes"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, TotalWrites]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(TotalWrites);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectRequests", "Requests"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, Requests = ZenStats.General.RequestCount]
		{
			if (bHaveStats)
			{
				return FText::AsNumber(Requests);
			}
			return LOCTEXT("UnavailableValue", "-");
		})
	];

	Row++;

	Panel->AddSlot(0, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, RowMargin))
		.Text(LOCTEXT("ProjectBadRequests", "Bad Requests"))
		];

	Panel->AddSlot(1, Row)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([bHaveStats, BadRequests = ZenStats.General.BadRequestCount]
			{
				if (bHaveStats)
				{
					return FText::AsNumber(BadRequests);
				}
	return LOCTEXT("UnavailableValue", "-");
			})
		];

	Row++;

	return Panel;
}

#undef LOCTEXT_NAMESPACE
