// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenServiceStatus.h"
#include "Experimental/ZenServerInterface.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ZenDashboard"

void SZenServiceStatus::Construct(const FArguments& InArgs)
{
	ZenServiceInstance = InArgs._ZenServiceInstance;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	UpdateState(0.0, 0.0);
	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SZenServiceStatus::UpdateState));
}

EActiveTimerReturnType SZenServiceStatus::UpdateState(double InCurrentTime, float InDeltaTime)
{
	uint32 LastStateIndex = ActiveStateIndex.load(std::memory_order_relaxed);
	uint32 NextStateIndex = (LastStateIndex + 1)%NumState;
	FState& NextState = State[NextStateIndex];
	NextState.bGotRunContext = UE::Zen::TryGetLocalServiceRunContext(NextState.RunContext);
	NextState.Version = UE::Zen::GetLocalServiceInstallVersion(false);
	if (NextState.bGotRunContext)
	{
		NextState.bIsRunning = UE::Zen::IsLocalServiceRunning(*NextState.RunContext.GetDataPath(), &NextState.LocalPort);
	}

	NextState.GCStatus = UE::Zen::FGCStatus{};
	NextState.bHaveStats = false;
	if (NextState.bIsRunning)
	{
		if (TSharedPtr<UE::Zen::FZenServiceInstance> ServiceInstance = ZenServiceInstance.Get())
		{
			ServiceInstance->GetGCStatus(NextState.GCStatus);
			if (ServiceInstance->GetCacheStats(NextState.ZenCacheStats) && ServiceInstance->GetProjectStats(NextState.ZenProjectStats))
			{
				NextState.bHaveStats = true;
			}
		}
	}

	ActiveStateIndex.compare_exchange_weak(LastStateIndex, NextStateIndex, std::memory_order_relaxed);

	return EActiveTimerReturnType::Continue;
}

FReply SZenServiceStatus::ExploreDataPath_OnClicked()
{
	const FState& CurrentState = GetCurrentState();
	if (CurrentState.bGotRunContext)
	{
		FString FullPath(FPaths::ConvertRelativePathToFull(CurrentState.RunContext.GetDataPath()));
		FPlatformProcess::ExploreFolder(*FullPath);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SZenServiceStatus::ExploreExecutablePath_OnClicked()
{
	const FState& CurrentState = GetCurrentState();
	if (CurrentState.bGotRunContext)
	{
		FString FullPath(FPaths::ConvertRelativePathToFull(FPaths::GetPath(CurrentState.RunContext.GetExecutable())));
		FPlatformProcess::ExploreFolder(*FullPath);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

const SZenServiceStatus::FState& SZenServiceStatus::GetCurrentState() const
{
	return State[ActiveStateIndex.load(std::memory_order_relaxed)];
}

TSharedRef<SWidget> SZenServiceStatus::GetGridPanel()
{
	TSharedRef<SGridPanel> Panel = SNew(SGridPanel);
	
	const static FNumberFormattingOptions SingleDecimalFormatting = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);

	int32 Row = 0;

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_Status", "Status"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([this]
		{
			const FState& CurrentState = GetCurrentState();
			FText ServiceStatus = LOCTEXT("ServiceStatus_NotInstalled", "Not installed");
			if (CurrentState.bGotRunContext)
			{
				if (CurrentState.bIsRunning && (CurrentState.LocalPort != 0))
				{
					ServiceStatus = LOCTEXT("ServiceStatus_Running", "Running");
				}
				else
				{
					ServiceStatus = LOCTEXT("ServiceStatus_Stopped", "Stopped");
				}
			}
			return ServiceStatus;
		})
	];

	++Row;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_Version", "Version"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([this]
		{
			const FState& CurrentState = GetCurrentState();
			return FText::FromString(CurrentState.Version);
		})
	];

	++Row;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_Port", "Port"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([this]
		{
			const FState& CurrentState = GetCurrentState();
			return CurrentState.LocalPort == 0 ? LOCTEXT("ServiceStatus_NoPortValue", "-") : FText::AsNumber(CurrentState.LocalPort, &FNumberFormattingOptions::DefaultNoGrouping());
		})
	];

	++Row;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_Browse", "Browse"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(350)
		.Padding(FMargin(ColumnMargin, RowMargin))
		.VAlign(VAlign_Bottom)
		[
			SNew(SHyperlink)
			.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
			.Text_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				FText BrowseText = LOCTEXT("ServiceStatus_NoBrowseValue", "-");
				if (CurrentState.bGotRunContext && CurrentState.bIsRunning && (CurrentState.LocalPort != 0))
				{
					BrowseText = FText::Format(LOCTEXT("ServiceStatus_BrowseLink", "http://localhost:{0}/dashboard/"),
						FText::AsNumber(CurrentState.LocalPort, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				return BrowseText;
			})
			.OnNavigate_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				if (CurrentState.bGotRunContext && CurrentState.bIsRunning && (CurrentState.LocalPort != 0))
				{
					FPlatformProcess::LaunchURL(*FString::Printf(TEXT("http://localhost:%d/dashboard/"), CurrentState.LocalPort), nullptr, nullptr);
				}
			})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(LOCTEXT("ServiceStatus_BrowseToolTip", "Browse the contents of zenserver in your web browser"))
			.IsEnabled_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				return CurrentState.bGotRunContext && CurrentState.bIsRunning && (CurrentState.LocalPort != 0);
			})
			.OnClicked_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				if (CurrentState.bGotRunContext && CurrentState.bIsRunning && (CurrentState.LocalPort != 0))
				{
					FPlatformProcess::LaunchURL(*FString::Printf(TEXT("http://localhost:%d/dashboard/"), CurrentState.LocalPort), nullptr, nullptr);
					return FReply::Handled();
				}
				return FReply::Unhandled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Zen.WebBrowser"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	++Row;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_GCStatus", "GC Status"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([this]
		{
			const FState& CurrentState = GetCurrentState();
			return CurrentState.LocalPort == 0 ? LOCTEXT("ServiceStatus_NoGCStatusValue", "-") : FText::FromString(CurrentState.GCStatus.Description);
		})
	];

	++Row;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_DiskSpace", "Disk Space"))
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.Text_Lambda([this]
		{
			const FState& CurrentState = GetCurrentState();
			const uint64 TotalDiskSpace = uint64(CurrentState.ZenCacheStats.General.Size.Disk) + uint64(CurrentState.ZenProjectStats.General.Size.Disk) + uint64(CurrentState.ZenCacheStats.CID.Size.Total);
			return CurrentState.bHaveStats ? FText::AsMemory(TotalDiskSpace, (TotalDiskSpace > 1024) ? &SingleDecimalFormatting : nullptr, nullptr, EMemoryUnitStandard::IEC) : LOCTEXT("UnavailableValue", "-");
		})
	];

	++Row;

	Panel->AddSlot(0, Row)
	.VAlign(VAlign_Bottom)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_DataPath", "Data path"))
	];

	Panel->AddSlot(1, Row)
	[

		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(350)
		.VAlign(VAlign_Bottom)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, RowMargin))
			.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
			.Text_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				return CurrentState.bGotRunContext ? FText::FromString(CurrentState.RunContext.GetDataPath()) : LOCTEXT("ServiceStatus_NoDataPathValue", "-");
			})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(LOCTEXT("ExploreDataPathButtonToolTip", "Explore the Data Path"))
			.IsEnabled_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				return CurrentState.bGotRunContext;
			})
			.OnClicked(this, &SZenServiceStatus::ExploreDataPath_OnClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Zen.FolderExplore"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	++Row;

	Panel->AddSlot(0, Row)
	.VAlign(VAlign_Bottom)
	[
		SNew(STextBlock)
		.Margin(FMargin(ColumnMargin, RowMargin))
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ServiceStatus_ExecutablePath", "Executable path"))
	];

	Panel->AddSlot(1, Row)
	[

		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		//.MaxWidth(350)
		.VAlign(VAlign_Bottom)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, RowMargin))
			.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
			.Text_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				return CurrentState.bGotRunContext ? FText::FromString(FPaths::GetPath(CurrentState.RunContext.GetExecutable())) : LOCTEXT("ServiceStatus_NoExecutablePathValue", "-");
			})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.ToolTipText(LOCTEXT("ExploreExecutablePathButtonToolTip", "Explore the Executable Path"))
			.IsEnabled_Lambda([this]
			{
				const FState& CurrentState = GetCurrentState();
				return CurrentState.bGotRunContext;
			})
			.OnClicked(this, &SZenServiceStatus::ExploreExecutablePath_OnClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Zen.FolderExplore"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	++Row;


	return Panel;
}

#undef LOCTEXT_NAMESPACE
