// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenWorkspaces.h"

#include "SZenServiceStatus.h"
#include "Experimental/ZenServerInterface.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/Application/SlateApplication.h"
#include "DesktopPlatformModule.h"
#include "Misc/MonitoredProcess.h"

#define LOCTEXT_NAMESPACE "ZenDashboard"

DEFINE_LOG_CATEGORY_STATIC(LogZenDashboardWorkpaces, Log, All);

namespace
{
	bool RunZenExe(FString Cmd)
	{
		FString ZenExePath = UE::Zen::GetLocalInstallUtilityPath();

		FMonitoredProcess ZenExeProcess(ZenExePath, Cmd, true);
		bool bLaunched = ZenExeProcess.Launch();
		if (!bLaunched)
		{
			UE_LOGF(LogZenDashboardWorkpaces, Warning, "Failed to launch zen utility: '%ls'.", *ZenExePath);
			return false;
		}

		const uint64 StartTime = FPlatformTime::Cycles64();
		while (ZenExeProcess.Update())
		{
			double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
			if (Duration > 10.f)
			{
				ZenExeProcess.Cancel(true);
				UE_LOGF(LogZenDashboardWorkpaces, Warning, "Cancelled launch of zen utility: '%ls %ls' due to timeout.", *ZenExePath, *Cmd);

				while (ZenExeProcess.Update())
				{
					Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
					if (Duration > 15.f)
					{
						UE_LOGF(LogZenDashboardWorkpaces, Warning, "Cancelled launch of zen utility: '%ls'. Failed waiting for termination.", *ZenExePath);
						break;
					}

					FPlatformProcess::Sleep(0.2f);
				}

				FString OutputString = ZenExeProcess.GetFullOutputWithoutDelegate();
				UE_LOGF(LogZenDashboardWorkpaces, Warning, "Launch of zen utility: '%ls' failed. Output: '%ls'", *ZenExePath, *OutputString);
				return false;
			}

			FPlatformProcess::Sleep(0.1f);
		}

		return true;
	}
}

class SZenNewWorkspace : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenNewWorkspace)
	{}
		SLATE_EVENT(FOnClicked, OnCloseClicked);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnCloseClicked = InArgs._OnCloseClicked;

		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.OnClicked(this, &SZenNewWorkspace::OnGoBack)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Back"))
				]
			]

			+ SVerticalBox::Slot()
			.Padding(0.f, 4.f)
			.AutoHeight()
			[
				SNew(SSeparator)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(2.f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NewWorkspacePath", "Workspace Path:"))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 0, 3)
				[
					SNew(SEditableTextBox)
					.Text(this, &SZenNewWorkspace::GetCurrentPath)

				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ContentPadding(FMargin(6.f, 2.f))
					.OnClicked(this, &SZenNewWorkspace::OnBrowseClick)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.FolderOpen"))
					]
				]
			]

			+ SVerticalBox::Slot()
			.Padding(0.f, 4.f)
			.AutoHeight()
			[
				SNew(SSeparator)
			]

			+ SVerticalBox::Slot()
			.Padding(0.f, 4.f)
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ContentPadding(FMargin(6.f, 2.f))
				.OnClicked(this, &SZenNewWorkspace::OnGoBack)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Check"))
					.ColorAndOpacity(FStyleColors::AccentGreen)
				]
			]
		];
	}

private:
	FText GetCurrentPath() const
	{
		return FText::GetEmpty();
	}

	FReply OnGoBack() const
	{
		if (OnCloseClicked.IsBound())
		{
			OnCloseClicked.Execute();
		}
		return FReply::Handled();
	}

	FReply OnBrowseClick()
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform)
		{
			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

			FString FolderName;
			const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
				ParentWindowHandle,
				LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(),
				"",
				FolderName
			);

			if (bFolderSelected)
			{
				if (!FolderName.EndsWith(TEXT("/")))
				{
					FolderName += TEXT("/");
				}

				SelectedDir = FolderName;
			}
		}

		return FReply::Handled();
	}

private:
	FOnClicked OnCloseClicked;
	FString SelectedDir;
};

class SZenWorkspaceShareRow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenWorkspaceShareRow)
		{}
		SLATE_ATTRIBUTE(UE::Zen::FZenWorkspaces::Workspace, WorkspaceInfo);
		SLATE_ATTRIBUTE(UE::Zen::FZenWorkspaces::Share, ShareInfo);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WorkspaceInfo = InArgs._WorkspaceInfo.Get();
		ShareInfo = InArgs._ShareInfo.Get();

		const float RowMargin = 10.f;
		const float ColumnMargin = 0.f;
		const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

		this->ChildSlot
		[
			SNew(SBorder)
			//.Padding(10.f, 10.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(2)
				[
					SNew(SGridPanel)
			
					+ SGridPanel::Slot(0, 0)
					[
						SNew(SBox)
						[
							SNew(STextBlock)
							.Font(TitleFont)
							.Margin(FMargin(RowMargin, ColumnMargin))
							.ColorAndOpacity(FStyleColors::AccentWhite)
							.Text(LOCTEXT("SharesId", "ShareId"))
						]
					]

					+ SGridPanel::Slot(1, 0)
					[
						SNew(STextBlock)
						.Margin(FMargin(RowMargin, ColumnMargin))
						.Text_Lambda([this]()
						{
							return FText::FromString(ShareInfo.Id);
						})
					]

					+ SGridPanel::Slot(0, 1)
					[
						SNew(STextBlock)
						.Margin(FMargin(RowMargin, ColumnMargin))
						.Font(TitleFont)
						.ColorAndOpacity(FStyleColors::AccentWhite)
						.Text(LOCTEXT("SharePath", "Path"))
					]

					+ SGridPanel::Slot(1, 1)
					[
						SNew(STextBlock)
						.Margin(FMargin(RowMargin, ColumnMargin))
						.Text_Lambda([this]()
						{
							return FText::FromString(ShareInfo.Dir);
						})
					]

					+ SGridPanel::Slot(0, 2)
					[
						SNew(STextBlock)
						.Margin(FMargin(RowMargin, ColumnMargin))
						.Font(TitleFont)
						.ColorAndOpacity(FStyleColors::AccentWhite)
						.Text(LOCTEXT("ShareAlias", "Alias"))
					]

					+ SGridPanel::Slot(1, 2)
					[
						SNew(STextBlock)
						.Margin(FMargin(RowMargin, ColumnMargin))
						.Text_Lambda([this]()
						{
							return FText::FromString(ShareInfo.Alias);
						})
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
						.OnClicked(this, &SZenWorkspaceShareRow::OnDeleteClicked)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.XCircle"))
							.ColorAndOpacity(FStyleColors::AccentRed)
						]
					]
				]
			]
		];
	}

	FReply OnDeleteClicked()
	{
		TStringBuilder<128> ShareRemoveCommand;
		ShareRemoveCommand << "workspace-share remove " << WorkspaceInfo.Id << " " << ShareInfo.Id;

		if (RunZenExe(ShareRemoveCommand.ToString()))
		{
			SetVisibility(EVisibility::Collapsed);
		}

		return FReply::Handled();
	}

private:
	UE::Zen::FZenWorkspaces::Workspace WorkspaceInfo;
	UE::Zen::FZenWorkspaces::Share ShareInfo;
};

class SZenWorkspaceRow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenWorkspaceRow)
		{}
		SLATE_ATTRIBUTE(UE::Zen::FZenWorkspaces::Workspace, WorkspaceInfo);
		SLATE_ATTRIBUTE(int32, IndexNumber)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WorkspaceInfo = InArgs._WorkspaceInfo.Get();
		IndexNumber = InArgs._IndexNumber.Get();

		const float RowMargin = 0.f;
		const float ColumnMargin = 10.f;
		const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Secondary"))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.FillWidth(1.0)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						[
							SNew(SGridPanel)

							// base path row
							+ SGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.ColorAndOpacity(FStyleColors::AccentWhite)
								.Font(TitleFont)
								.Text(LOCTEXT("ZenWorkspaces_Path", "Root dir"))
							]
							+ SGridPanel::Slot(1, 0)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.Text_Lambda([this]
								{
									return FText::FromString(WorkspaceInfo.BaseDir);
								})
							]

							// id row
							+ SGridPanel::Slot(0, 1)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.ColorAndOpacity(FStyleColors::AccentWhite)
								.Font(TitleFont)
								.Text(LOCTEXT("ZenWorkspaces_Id", "Id"))
							]
							+ SGridPanel::Slot(1, 1)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.Text_Lambda([this]
								{
									return FText::FromString(WorkspaceInfo.Id);
								})
							]

							+ SGridPanel::Slot(0, 2)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.ColorAndOpacity(FStyleColors::AccentWhite)
								.Font(TitleFont)
								.Text(LOCTEXT("ZenWorkspaces_Dynamic", "Dynamic shares"))
								.ToolTipText(LOCTEXT("WorkspacesDynamic_Tooltip", "Does this workspace allow for creating shares using the /ws http endpoint"))
							]
							+ SGridPanel::Slot(1, 2)
							[
								SNew(STextBlock)
								.Margin(FMargin(ColumnMargin, RowMargin))
								.Text_Lambda([this]
								{
									FText DynamicStatus = LOCTEXT("WorkspaceStatus_Disabled", "Disabled");
									if (WorkspaceInfo.bDynamicShare)
									{
										DynamicStatus = LOCTEXT("WorkspaceStatus_Enabled", "Enabled");
									}

									return DynamicStatus;
								})
							]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 10.f, 0.f)
						[
							SNew(SBox)
							[
								SNew(SButton)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.OnClicked(this, &SZenWorkspaceRow::OnDeleteClicked)
								.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
								[
									SNew(SImage)
									.Image(FAppStyle::Get().GetBrush("GenericCommands.Delete"))
									.ColorAndOpacity(FStyleColors::AccentRed)
								]
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.f, 8.f)
					[
						ConstructShareList()
					]
				]
			]
		];
	}

	FReply OnDeleteClicked()
	{
		TStringBuilder<128> WorkspaceRemoveCommand;
		WorkspaceRemoveCommand << "workspace remove " << WorkspaceInfo.Id;

		for (int Idx = 0; Idx < WorkspaceInfo.WorkspaceShares.Num(); ++Idx)
		{
			TStringBuilder<128> ShareRemoveCommand;
			ShareRemoveCommand << "workspace-share remove " << WorkspaceInfo.Id << " " << WorkspaceInfo.WorkspaceShares[Idx].Id;

			RunZenExe(ShareRemoveCommand.ToString());
		}

		if (RunZenExe(WorkspaceRemoveCommand.ToString()))
		{
			SetVisibility(EVisibility::Collapsed);
		}

		return FReply::Handled();
	}

private:
	TSharedRef<SWidget> ConstructShareList()
	{
		const float RowMargin = 0.f;
		const float ColumnMargin = 10.f;
		const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

		TSharedRef<SVerticalBox> Widget = SNew(SVerticalBox);

		if (WorkspaceInfo.WorkspaceShares.IsEmpty())
		{
			return Widget;
		}

		for (int32 Idx = 0; Idx < WorkspaceInfo.WorkspaceShares.Num(); ++Idx)
		{
			Widget->AddSlot()
			.AutoHeight()
			.Padding(0.f, 1.f)
			[
				SNew(SZenWorkspaceShareRow)
				.WorkspaceInfo(WorkspaceInfo)
				.ShareInfo(WorkspaceInfo.WorkspaceShares[Idx])
			];
		}

		TSharedRef<SBorder> Result = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
					.Font(TitleFont)
					.Margin(FMargin(RowMargin, ColumnMargin))
					.ColorAndOpacity(FStyleColors::AccentWhite)
					.Text(LOCTEXT("Shares", "Shared folders:"))
			]

			+SVerticalBox::Slot()
			[
				Widget
			]
		];

		return Result;
	}

private:
	UE::Zen::FZenWorkspaces::Workspace WorkspaceInfo;
	int32 IndexNumber;
	bool bVisible = true;
};

void SZenWorkspaces::Construct(const FArguments& InArgs)
{
	ZenServiceInstance = InArgs._ZenServiceInstance;
	UpdateFrequency = InArgs._UpdateFrequency.Get();

	const FSlateColor TitleColor = FStyleColors::AccentWhite;

	this->ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f)
			.HAlign(HAlign_Fill)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("ZenWorkspacesExpand", "Workspaces"))
				.OnClicked(this, &SZenWorkspaces::ZenWorkspacesArea_Toggle)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(this, &SZenWorkspaces::ZenWorkspacesArea_Icon)
						.ColorAndOpacity(FStyleColors::White)
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.f)
					[
						SNew(STextBlock)
						.ColorAndOpacity(FStyleColors::White)
						.Text(LOCTEXT("ZenWorkspaces", "Workspaces"))
					]
				]
			]

			+ SVerticalBox::Slot()
			.MaxHeight(400)
			.Padding(0.f, 8.f)
			[
				SAssignNew(WidgetSwitcher, SWidgetSwitcher)
				.Visibility(EVisibility::Collapsed)

				+ SWidgetSwitcher::Slot()
				[
					SAssignNew(WorkspaceArea, SVerticalBox)

					+ SVerticalBox::Slot()
					.Expose(GridSlot)
					[
						GetWorkspaceList()
					]
				]

				+ SWidgetSwitcher::Slot()
				[
					SNew(SZenNewWorkspace)
					.OnCloseClicked(this, &SZenWorkspaces::OnBackToMainWidget)
				]
			]
		];

	UpdateWorkspaces(0, true);
}

TSharedRef<SWidget> SZenWorkspaces::GetWorkspaceList()
{
	TSharedPtr<SScrollBox> WorkspaceItems = SNew(SScrollBox);

	const float RowMargin = 0.f;
	const float ColumnMargin = 10.f;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	int32 Idx = 1;
	for (UE::Zen::FZenWorkspaces::Workspace& Workspace : Workspaces.ZenWorkspaces)
	{
		WorkspaceItems->AddSlot()
		.Padding(16.f, 4.f)
		.VAlign(VAlign_Center)
		[
			SNew(SZenWorkspaceRow)
			.WorkspaceInfo(Workspace)
			.IndexNumber(Idx++)
		];
	}

	return WorkspaceItems.ToSharedRef();
}

FReply SZenWorkspaces::ZenWorkspacesArea_Toggle() const
{
	if (WidgetSwitcher)
	{
		if (WidgetSwitcher->GetVisibility() == EVisibility::Visible)
		{
			WidgetSwitcher->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			WidgetSwitcher->SetVisibility(EVisibility::Visible);
		}
	}

	return FReply::Handled();
}

FReply SZenWorkspaces::OnBackToMainWidget()
{
	WidgetSwitcher->SetActiveWidgetIndex(0);
	return FReply::Handled();
}

FReply SZenWorkspaces::OnCreateNewWorkspaceClicked()
{
	WidgetSwitcher->SetActiveWidgetIndex(1);
	return FReply::Handled();
}

const FSlateBrush* SZenWorkspaces::ZenWorkspacesArea_Icon() const
{
	return (WidgetSwitcher && WidgetSwitcher->GetVisibility() == EVisibility::Visible) ?
		FAppStyle::Get().GetBrush("Icons.ChevronDown") :
		FAppStyle::Get().GetBrush("Icons.ChevronRight");
}

void SZenWorkspaces::UpdateWorkspaces(float InDeltaTime, bool bForce)
{
	static double AccumulatedDeltaTime = 0.f;

	AccumulatedDeltaTime += InDeltaTime;
	if (AccumulatedDeltaTime >= UpdateFrequency || bForce)
	{
		AccumulatedDeltaTime = 0.f;

		UE::Zen::FZenWorkspaces RecentWorkspaces;
		if (TSharedPtr<UE::Zen::FZenServiceInstance> ServiceInstance = ZenServiceInstance.Get())
		{
			ServiceInstance->GetWorkspaces(RecentWorkspaces);
		}

		if (RecentWorkspaces == Workspaces)
		{
			return;
		}

		Workspaces = RecentWorkspaces;
		if (GridSlot)
		{
			(*GridSlot)
				[
					GetWorkspaceList()
				];
		}

		SlatePrepass(GetPrepassLayoutScaleMultiplier());
	}
}

#undef LOCTEXT_NAMESPACE
