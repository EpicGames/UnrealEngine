// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenServiceInstanceManager.h"
#include "Experimental/ZenServerInterface.h"


namespace UE::Zen
{

TSharedPtr<FZenServiceInstance> FServiceInstanceManager::GetZenServiceInstance() const
{
	UE::Zen::FZenLocalServiceRunContext RunContext;
	if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
	{
		uint16 LocalPort = 0;
		if (UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath(), &LocalPort) && (LocalPort != 0))
		{
			if (LocalPort != CurrentPort)
			{
				CurrentPort = LocalPort;
				UE::Zen::FServiceSettings ServiceSettings;
				ServiceSettings.SettingsVariant.Emplace<UE::Zen::FServiceConnectSettings>();
				UE::Zen::FServiceConnectSettings& ConnectExistingSettings = ServiceSettings.SettingsVariant.Get<UE::Zen::FServiceConnectSettings>();
				ConnectExistingSettings.HostName = TEXT("localhost");
				ConnectExistingSettings.Port = LocalPort;
				CurrentInstance = MakeShared<FZenServiceInstance>(MoveTemp(ServiceSettings));
			}

			return CurrentInstance;
		}
	}
	return nullptr;
}

} // namespace UE::Zen
