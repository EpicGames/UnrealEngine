// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StorageServerWidgets : ModuleRules
{
	public StorageServerWidgets(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"InputCore",
				"Json",
				"Projects",	
				"Slate",
				"SlateCore",
				"ToolWidgets",
				"Zen"
			});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
